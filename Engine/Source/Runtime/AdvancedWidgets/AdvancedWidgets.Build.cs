// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AdvancedWidgets : ModuleRules
{
	public AdvancedWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"SlateCore",
				"UMG"
			}
		);
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"InputCore",
				"Slate",
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("UnrealEd");
			PrivateDependencyModuleNames.Add("PropertyEditor");
		}

		bDisableAutoRTFMInstrumentation = true;
	}
}
