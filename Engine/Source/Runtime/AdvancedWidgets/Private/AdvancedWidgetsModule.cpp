// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedWidgetsModule.h"

#include "Framework/PropertyViewer/PropertyValueFactory.h"
#include "Modules/ModuleManager.h"
#include "Styling/AdvancedWidgetsStyle.h"


IMPLEMENT_MODULE(FAdvancedWidgetsModule, AdvancedWidgets);

namespace UE::AdvancedWidgets::Private
{
	static const FName NAME_AdvancedWidgets = "AdvancedWidgets";
}

FAdvancedWidgetsModule& FAdvancedWidgetsModule::GetModule()
{
	return FModuleManager::LoadModuleChecked<FAdvancedWidgetsModule>(UE::AdvancedWidgets::Private::NAME_AdvancedWidgets);
}

FAdvancedWidgetsModule* FAdvancedWidgetsModule::GetModulePtr()
{
	return FModuleManager::GetModulePtr<FAdvancedWidgetsModule>(UE::AdvancedWidgets::Private::NAME_AdvancedWidgets);
}

void FAdvancedWidgetsModule::StartupModule()
{
	UE::AdvancedWidgets::FAdvancedWidgetsStyle::Create();
	PropertyValueFactory = MakeUnique<UE::PropertyViewer::FPropertyValueFactory>();
}

void FAdvancedWidgetsModule::ShutdownModule()
{
	UE::AdvancedWidgets::FAdvancedWidgetsStyle::Destroy();
}

UE::PropertyViewer::FPropertyValueFactory& FAdvancedWidgetsModule::GetPropertyValueFactory() const
{
	return *(PropertyValueFactory.Get());
}
