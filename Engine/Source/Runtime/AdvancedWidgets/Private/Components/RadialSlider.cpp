// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/RadialSlider.h"
#include "Widgets/SRadialSlider.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RadialSlider)

#define LOCTEXT_NAMESPACE "UMG"

/////////////////////////////////////////////////////
// URadialSlider

static FSliderStyle* DefaultSliderStyle = nullptr;

URadialSlider::URadialSlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseCustomDefaultValue = false;
	CustomDefaultValue = 0.0f;

	//Set a default range of 0 to 1 in a linear curve.
	SliderRange.GetRichCurve()->AddKey(0.0f, 0.0f);
	SliderRange.GetRichCurve()->AddKey(1.0f, 1.0f);

	SliderHandleStartAngle = 60.0f;
	SliderHandleEndAngle = 300.0f;
	AngularOffset = 0.0f;
	HandStartEndRatio = FVector2D(0.0f, 1.0f);
	SliderBarColor = FLinearColor::Gray;
	SliderProgressColor = FLinearColor::White;
	SliderHandleColor = FLinearColor::White;
	CenterBackgroundColor = FLinearColor::Transparent;
	StepSize = 0.01f;
	IsFocusable = true;
	MouseUsesStep = false;
	RequiresControllerLock = true;
	UseVerticalDrag = false;
	ShowSliderHandle = true;
	ShowSliderHand = false;

	if (DefaultSliderStyle == nullptr)
	{
		// HACK: THIS SHOULD NOT COME FROM CORESTYLE AND SHOULD INSTEAD BE DEFINED BY ENGINE TEXTURES/PROJECT SETTINGS
		DefaultSliderStyle = new FSliderStyle(FCoreStyle::Get().GetWidgetStyle<FSliderStyle>("Slider")); //TODO: Create bespoke SlateStyle for Radial Widget

		// Unlink UMG default colors from the editor settings colors.
		DefaultSliderStyle->UnlinkColors();
	}

	WidgetStyle = *DefaultSliderStyle;

#if WITH_EDITORONLY_DATA
	AccessibleBehavior = ESlateAccessibleBehavior::Summary;
	bCanChildrenBeAccessible = false;
#endif
}

TSharedRef<SWidget> URadialSlider::RebuildWidget()
{
	MyRadialSlider = SNew(SRadialSlider)
		.Style(&WidgetStyle)
		.IsFocusable(IsFocusable)
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd))
		.OnControllerCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnControllerCaptureBegin))
		.OnControllerCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnControllerCaptureEnd))
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged));

	return MyRadialSlider.ToSharedRef();
}

void URadialSlider::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!MyRadialSlider.IsValid())
	{
		return;
	}

	FRichCurve* OurCurve = SliderRange.GetRichCurve();
	for (int32 i = 0; i < OurCurve->Keys.Num() - 1; i++)
	{
		if (i > 0)
		{
			const float PreviousCurveKeyTime = OurCurve->Keys[i - 1].Time;
			if (OurCurve->Keys[i].Time < PreviousCurveKeyTime)
			{
				OurCurve->Keys[i].Time = PreviousCurveKeyTime;
			}

			const float PreviousCurveKeyValue = OurCurve->Keys[i - 1].Value;
			if (OurCurve->Keys[i].Value < PreviousCurveKeyValue)
			{
				OurCurve->Keys[i].Value = PreviousCurveKeyValue;
			}
		}

		const float NextCurveKeyTime = OurCurve->Keys[i + 1].Time;
		if (OurCurve->Keys[i].Time > NextCurveKeyTime)
		{
			OurCurve->Keys[i].Time = NextCurveKeyTime;
		}

		const float NextCurveKeyValue = OurCurve->Keys[i + 1].Value;
		if (OurCurve->Keys[i].Value > NextCurveKeyValue)
		{
			OurCurve->Keys[i].Value = NextCurveKeyValue;
		}
	}

	TAttribute<float> ValueBinding = PROPERTY_BINDING(float, Value);
	
	MyRadialSlider->SetMouseUsesStep(MouseUsesStep);
	MyRadialSlider->SetRequiresControllerLock(RequiresControllerLock);
	MyRadialSlider->SetSliderBarColor(SliderBarColor);
	MyRadialSlider->SetSliderProgressColor(SliderProgressColor);
	MyRadialSlider->SetSliderHandleColor(SliderHandleColor);
	MyRadialSlider->SetCenterBackgroundColor(CenterBackgroundColor);
	MyRadialSlider->SetValue(ValueBinding);
	MyRadialSlider->SetUseCustomDefaultValue(bUseCustomDefaultValue);
	MyRadialSlider->SetCustomDefaultValue(CustomDefaultValue);
	MyRadialSlider->SetSliderRange(SliderRange);
	MyRadialSlider->SetValueTags(ValueTags);
	MyRadialSlider->SetSliderHandleStartAngleAndSliderHandleEndAngle(SliderHandleStartAngle, SliderHandleEndAngle);
	MyRadialSlider->SetAngularOffset(AngularOffset);
	MyRadialSlider->SetHandStartEndRatio(HandStartEndRatio);
	MyRadialSlider->SetLocked(Locked);
	MyRadialSlider->SetStepSize(StepSize);
	MyRadialSlider->SetUseVerticalDrag(UseVerticalDrag);
	MyRadialSlider->SetShowSliderHandle(ShowSliderHandle);
	MyRadialSlider->SetShowSliderHand(ShowSliderHand);
}

void URadialSlider::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyRadialSlider.Reset();
}

void URadialSlider::HandleOnValueChanged(float InValue)
{
	OnValueChanged.Broadcast(InValue);
}

void URadialSlider::HandleOnMouseCaptureBegin()
{
	OnMouseCaptureBegin.Broadcast();
}

void URadialSlider::HandleOnMouseCaptureEnd()
{
	OnMouseCaptureEnd.Broadcast();
}

void URadialSlider::HandleOnControllerCaptureBegin()
{
	OnControllerCaptureBegin.Broadcast();
}

void URadialSlider::HandleOnControllerCaptureEnd()
{
	OnControllerCaptureEnd.Broadcast();
}

float URadialSlider::GetValue() const
{
	if (MyRadialSlider.IsValid() )
	{
		return MyRadialSlider->GetValue();
	}

	return Value;
}

float URadialSlider::GetCustomDefaultValue() const
{
	if (MyRadialSlider.IsValid())
	{
		return MyRadialSlider->GetCustomDefaultValue();
	}

	return Value;
}

float URadialSlider::GetNormalizedSliderHandlePosition() const
{
	if (MyRadialSlider.IsValid())
	{
		return MyRadialSlider->GetNormalizedSliderHandlePosition();
	}
	
	return 0.0f;
}

void URadialSlider::SetValue(float InValue)
{
	Value = InValue;
	if (MyRadialSlider.IsValid() )
	{
		MyRadialSlider->SetValue(InValue);
	}
}

void URadialSlider::SetCustomDefaultValue(float InValue)
{
	CustomDefaultValue = InValue;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetCustomDefaultValue(InValue);
	}
}

void URadialSlider::SetSliderRange(const FRuntimeFloatCurve& InSliderRange)
{
	SliderRange = InSliderRange;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetSliderRange(InSliderRange);
	}
}

void URadialSlider::SetSliderHandleStartAngle(float InValue)
{
	SliderHandleStartAngle = InValue;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetSliderHandleStartAngleAndSliderHandleEndAngle(SliderHandleStartAngle, SliderHandleEndAngle);
	}
}

void URadialSlider::SetSliderHandleEndAngle(float InValue)
{
	SliderHandleEndAngle = InValue;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetSliderHandleStartAngleAndSliderHandleEndAngle(SliderHandleStartAngle, SliderHandleEndAngle);
	}
}

void URadialSlider::SetAngularOffset(float InValue)
{
	AngularOffset = InValue;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetAngularOffset(AngularOffset);
	}
}

void URadialSlider::SetHandStartEndRatio(FVector2D InValue)
{
	// value will be clamped by radial slider itself if necessary
	HandStartEndRatio = InValue;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetHandStartEndRatio(HandStartEndRatio);
	}
}

void URadialSlider::SetValueTags(const TArray<float>& InValueTags)
{
	ValueTags = InValueTags;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetValueTags(ValueTags);
	}
}

void URadialSlider::SetLocked(bool InLocked)
{
	Locked = InLocked;
	if (MyRadialSlider.IsValid() )
	{
		MyRadialSlider->SetLocked(InLocked);
	}
}

void URadialSlider::SetStepSize(float InValue)
{
	StepSize = InValue;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetStepSize(InValue);
	}
}

void URadialSlider::SetCenterBackgroundColor(FLinearColor InValue)
{
	CenterBackgroundColor = InValue;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetCenterBackgroundColor(InValue);
	}
}

void URadialSlider::SetSliderBarColor(FLinearColor InValue)
{
	SliderBarColor = InValue;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetSliderBarColor(InValue);
	}
}

void URadialSlider::SetSliderProgressColor(FLinearColor InValue)
{
	SliderProgressColor = InValue;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetSliderProgressColor(InValue);
	}
}

void URadialSlider::SetSliderHandleColor(FLinearColor InValue)
{
	SliderHandleColor = InValue;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetSliderHandleColor(InValue);
	}
}

void URadialSlider::SetUseVerticalDrag(bool InUseVerticalDrag)
{
	UseVerticalDrag = InUseVerticalDrag;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetUseVerticalDrag(InUseVerticalDrag);
	}
}

void URadialSlider::SetShowSliderHandle(bool InShowSliderHandle)
{
	ShowSliderHandle = InShowSliderHandle;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetShowSliderHandle(InShowSliderHandle);
	}
}

void URadialSlider::SetShowSliderHand(bool InShowSliderHand)
{
	ShowSliderHand = InShowSliderHand;
	if (MyRadialSlider.IsValid())
	{
		MyRadialSlider->SetShowSliderHand(InShowSliderHand);
	}
}

#if WITH_ACCESSIBILITY
TSharedPtr<SWidget> URadialSlider::GetAccessibleWidget() const
{
	return MyRadialSlider;
}
#endif

#if WITH_EDITOR

const FText URadialSlider::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}

#endif

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE

