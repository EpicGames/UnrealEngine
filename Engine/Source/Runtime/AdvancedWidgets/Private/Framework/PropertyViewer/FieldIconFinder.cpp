// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/PropertyViewer/FieldIconFinder.h"

#include "Styling/AdvancedWidgetsStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"


namespace UE::PropertyViewer
{

namespace Private
{

FSlateColor GetColor(const FFieldColorSettings& Settings, const FProperty* Property)
{
	if (CastField<const FClassProperty>(Property))
	{
		return Settings.ClassTypeColor;
	}
	if (CastField<const FObjectPropertyBase>(Property))
	{
		return Settings.ObjectTypeColor;
	}
	if (CastField<const FInterfaceProperty>(Property))
	{
		return Settings.InterfaceTypeColor;
	}
	if (const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(Property))
	{
		if (NumericProperty->IsEnum())
		{
			return Settings.EnumTypeColor;
		}
		if (NumericProperty->IsFloatingPoint())
		{
			return Settings.FloatTypeColor;
		}
		if (NumericProperty->IsInteger())
		{
			return Settings.IntTypeColor;
		}
	}
	if (CastField<const FBoolProperty>(Property))
	{
		return Settings.BooleanTypeColor;
	}
	if (CastField<const FEnumProperty>(Property))
	{
		return Settings.EnumTypeColor;
	}
	if (CastField<const FStrProperty>(Property))
	{
		return Settings.StringTypeColor;
	}
	if (CastField<const FTextProperty>(Property))
	{
		return Settings.StringTypeColor;
	}
	if (CastField<const FNameProperty>(Property))
	{
		return Settings.TextTypeColor;
	}
	if (CastField<const FMulticastDelegateProperty>(Property) || CastField<const FDelegateProperty>(Property))
	{
		return Settings.DelegateTypeColor;
	}
	if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
	{
		if (const FLinearColor* FoundColor = Settings.StructColors.Find(StructProperty->Struct->GetStructPathName().ToString()))
		{
			return *FoundColor;
		}
		return Settings.DefaultStructTypeColor;
	}
	return Settings.DefaultTypeColor;
}

} //namespace Private


const FSlateBrush* FFieldIconFinder::GetIcon(const UObject* Object)
{
	if (const UFunction* Function = Cast<const UFunction>(Object))
	{
		FFieldIconArray FunctionResult = GetFunctionIcon(Function);
		if (FunctionResult.Num() > 0)
		{
			return FunctionResult[0].Icon;
		}
	}
	if (const UClass* Class = Cast<const UClass>(Object))
	{
		return FSlateIconFinder::FindIconBrushForClass(Class);
	}
	return FSlateIconFinder::FindIconBrushForClass(Object->GetClass());
}


FFieldIconFinder::FFieldIconArray FFieldIconFinder::GetFunctionIcon(const UFunction* Function)
{
	return GetFunctionIcon(Function, ::UE::AdvancedWidgets::FAdvancedWidgetsStyle::GetColorSettings());
}


FFieldIconFinder::FFieldIconArray FFieldIconFinder::GetFunctionIcon(const UFunction* Function, const FFieldColorSettings& Settings)
{
	check(Function);

	FFieldIcon Icon;
	if (UFunction* OverrideFunc = Function->GetSuperFunction())
	{
		const bool bIsPureFunction = OverrideFunc && OverrideFunc->HasAnyFunctionFlags(FUNC_BlueprintPure);
		Icon.Icon = FAppStyle::GetBrush(bIsPureFunction ? TEXT("GraphEditor.OverridePureFunction_16x") : TEXT("GraphEditor.OverrideFunction_16x"));
	}
	else
	{
		const bool bIsPureFunction = Function->HasAnyFunctionFlags(FUNC_BlueprintPure);
		Icon.Icon = FAppStyle::GetBrush(bIsPureFunction ? TEXT("GraphEditor.PureFunction_16x") : TEXT("GraphEditor.Function_16x"));
	}

	Icon.Color = FLinearColor::White;
	if (FProperty* ReturnProperty = Function->GetReturnProperty())
	{
		FFieldIconArray ReturnValueResult = GetPropertyIcon(ReturnProperty, Settings);
		if (ReturnValueResult.Num() > 0)
		{
			Icon.Color = ReturnValueResult[0].Color;
		}
	}

	FFieldIconArray Result;
	Result.Add(Icon);
	return Result;
}


FFieldIconFinder::FFieldIconArray FFieldIconFinder::GetPropertyIcon(const FProperty* Property)
{
	return GetPropertyIcon(Property, ::UE::AdvancedWidgets::FAdvancedWidgetsStyle::GetColorSettings());
}


FFieldIconFinder::FFieldIconArray FFieldIconFinder::GetPropertyIcon(const FProperty* Property, const FFieldColorSettings& Settings)
{
	check(Property);

	FFieldIconArray Result;

	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		FFieldIcon Icon;
		Icon.Icon = FAppStyle::GetBrush(TEXT("Kismet.VariableList.ArrayTypeIcon"));
		Icon.Color = Private::GetColor(Settings, ArrayProperty->Inner);
		Result.Add(Icon);
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
	{
		FFieldIcon Icon;
		Icon.Icon = FAppStyle::GetBrush(TEXT("Kismet.VariableList.MapKeyTypeIcon"));
		Icon.Color = Private::GetColor(Settings, MapProperty->GetKeyProperty());
		Result.Add(Icon);
		Icon.Icon = FAppStyle::GetBrush(TEXT("Kismet.VariableList.MapValueTypeIcon"));
		Icon.Color = Private::GetColor(Settings, MapProperty->GetValueProperty());
		Result.Add(Icon);
	}
	else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
	{
		FFieldIcon Icon;
		Icon.Icon = FAppStyle::GetBrush(TEXT("Kismet.VariableList.SetTypeIcon"));
		Icon.Color = Private::GetColor(Settings, SetProperty->ElementProp);
		Result.Add(Icon);
	}
	else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
	{
		FFieldIcon Icon;
		Icon.Icon = FSlateIconFinder::FindIconBrushForClass(ObjectProperty->PropertyClass);
		Icon.Color = Private::GetColor(Settings, Property);
		Result.Add(Icon);
	}
	else
	{
		FFieldIcon Icon;
		Icon.Icon = FAppStyle::GetBrush(TEXT("Kismet.VariableList.TypeIcon"));
		Icon.Color = Private::GetColor(Settings, Property);
		Result.Add(Icon);
	}

	//return FAppStyle::GetBrush(TEXT("Kismet.AllClasses.VariableIcon");
	return Result;
}

} //namespace
