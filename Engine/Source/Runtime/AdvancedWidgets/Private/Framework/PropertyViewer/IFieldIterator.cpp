// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/PropertyViewer/IFieldIterator.h"

#include "UObject/Class.h"
#include "UObject/UnrealType.h"

namespace UE::PropertyViewer
{

/**
 * FFieldIterator_BlueprintVisible
 */
TArray<FFieldVariant> FFieldIterator_BlueprintVisible::GetFields(const UStruct* Struct, const FName FieldName, const UStruct* ContainerStruct)  const
{
	TArray<FFieldVariant> Result;
	for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		if (Property->HasAnyPropertyFlags(CPF_BlueprintVisible | CPF_BlueprintAssignable | CPF_Parm)
			&& !Property->HasAnyPropertyFlags(CPF_Deprecated | CPF_EditorOnly))
		{
			Result.Add(FFieldVariant(Property));
		}
	}
	for (TFieldIterator<UFunction> FunctionItt(Struct, EFieldIteratorFlags::IncludeSuper); FunctionItt; ++FunctionItt)
	{
		UFunction* Function = *FunctionItt;
		if (Function->HasAnyFunctionFlags(FUNC_BlueprintCallable))
		{
			Result.Add(FFieldVariant(Function));
		}
	}
	return Result;
}

} //namespace
