// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/PropertyViewer/PropertyPath.h"
#include "UObject/UnrealType.h"

namespace UE::PropertyViewer
{

FPropertyPath::FPropertyPath(UObject* Object, const FProperty* Property)
	: TopLevelContainer_Object(Object)
{
	Properties.Add(Property);
}


FPropertyPath::FPropertyPath(UObject* Object, FPropertyArray InProperties)
	: TopLevelContainer_Object(Object)
	, Properties(MoveTemp(InProperties))
{}


FPropertyPath::FPropertyPath(const UScriptStruct* ScriptStruct, void* Data, const FProperty* Property)
	: TopLevelContainer_ScriptStruct(ScriptStruct)
	, TopLevelContainer_ScriptStructData(Data)
{
	Properties.Add(Property);
}


FPropertyPath::FPropertyPath(const UScriptStruct* ScriptStruct, void* Data, FPropertyArray InProperties)
	: TopLevelContainer_ScriptStruct(ScriptStruct)
	, TopLevelContainer_ScriptStructData(Data)
	, Properties(MoveTemp(InProperties))
{}


void* FPropertyPath::GetContainerPtr()
{
	void* Current = nullptr;
	if (UObject* ObjectInstance = TopLevelContainer_Object.Get())
	{
		bool bInvalidClass = ObjectInstance->GetClass()->HasAnyClassFlags(EClassFlags::CLASS_NewerVersionExists);
		if (!bInvalidClass)
		{
			Current = reinterpret_cast<void*>(ObjectInstance);
		}
	}
	else if (const UScriptStruct* StructInstance = TopLevelContainer_ScriptStruct.Get())
	{
		bool bInvalidClass = (StructInstance->StructFlags & (EStructFlags::STRUCT_Trashed)) != 0;
		if (!bInvalidClass)
		{
			Current = TopLevelContainer_ScriptStructData;
		}
	}

	for (int32 Index = 0; Index < Properties.Num() - 1; ++Index)
	{
		if (Current == nullptr)
		{
			return nullptr;
		}

		const FProperty* CurrentProperty = Properties[Index];
		if (CurrentProperty == nullptr)
		{
			return nullptr;
		}

		if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(CurrentProperty))
		{
			Current = reinterpret_cast<void*>(ObjectProperty->GetObjectPropertyValue_InContainer(Current));
		}
		else
		{
			// We do not use the Getter on purpose.
			Current = CurrentProperty->ContainerPtrToValuePtr<void>(Current);
		}
	}

	return Current;
}


const void* FPropertyPath::GetContainerPtr() const
{
	return const_cast<FPropertyPath*>(this)->GetContainerPtr();
}

} //namespace
