// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/PropertyViewer/PropertyValueFactory.h"
#include "HAL/PlatformMemory.h"
#include "Misc/MessageDialog.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/PropertyViewer/SBoolPropertyValue.h"
#include "Widgets/PropertyViewer/SEnumPropertyValue.h"
#include "Widgets/PropertyViewer/SDefaultPropertyValue.h"
#include "Widgets/PropertyViewer/SNumericPropertyValue.h"
#include "Widgets/PropertyViewer/SStringPropertyValue.h"
#include "Widgets/PropertyViewer/SObjectPropertyValue.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "PropertyValueFactory"

namespace UE::PropertyViewer
{
namespace Private
{

TSharedPtr<SWidget> GenerateNumericEditor(FPropertyValueFactory::FGenerateArgs Args)
{
	if (const FProperty* Property = Args.Path.GetLastProperty())
	{
		if (CastFieldChecked<const FNumericProperty>(Property)->IsEnum())
		{
			return SEnumPropertyValue::CreateInstance(Args);
		}
	}
	return SNumericPropertyValue::CreateInstance(Args);
}


struct FPropertyViewerMapItem
{
	FPropertyValueFactory::FOnGenerate OnGenerate;
	FPropertyValueFactory::FHandle Handle;
};


class FPropertyValueFactoryImpl
{
public:
	/** For FProperty. Use FFieldClass::Name */
	TMap<const FFieldClass*, FPropertyViewerMapItem> RegisteredFieldClassEditor;
	/** For UScriptStruct or UClass */
	TMap<FTopLevelAssetPath, FPropertyViewerMapItem> RegisteredFieldEditor;
};

} //namespace


FPropertyValueFactory::FPropertyValueFactory()
{
	Impl = MakePimpl<Private::FPropertyValueFactoryImpl>();
	Register(FNumericProperty::StaticClass(), FPropertyValueFactory::FOnGenerate::CreateStatic(Private::GenerateNumericEditor));
	Register(FBoolProperty::StaticClass(), FPropertyValueFactory::FOnGenerate::CreateStatic(SBoolPropertyValue::CreateInstance));
	Register(FEnumProperty::StaticClass(), FPropertyValueFactory::FOnGenerate::CreateStatic(SEnumPropertyValue::CreateInstance));
	Register(FStrProperty::StaticClass(), FPropertyValueFactory::FOnGenerate::CreateStatic(SStringPropertyValue::CreateInstance));
	Register(FTextProperty::StaticClass(), FPropertyValueFactory::FOnGenerate::CreateStatic(SStringPropertyValue::CreateInstance));
	Register(FNameProperty::StaticClass(), FPropertyValueFactory::FOnGenerate::CreateStatic(SStringPropertyValue::CreateInstance));
	Register(UObject::StaticClass(), FPropertyValueFactory::FOnGenerate::CreateStatic(SObjectPropertyValue::CreateInstance));
}

FProperty* GetReturnProperty(UFunction* InFunction)
{
	if (FProperty* ReturnProperty = InFunction->GetReturnProperty())
	{
		return ReturnProperty;
	}

	if (InFunction->HasAllFunctionFlags(FUNC_HasOutParms))
	{
		for (TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & CPF_Parm); ++It)
		{
			if (It->HasAllPropertyFlags(CPF_OutParm) && !It->HasAnyPropertyFlags(CPF_Deprecated | CPF_EditorOnly | CPF_ConstParm | CPF_ReferenceParm))
			{
				return *It;
			}
		}
	}

	return nullptr;
}

UObject* GetUObjectFromContainer(void* Container)
{
	UObject* Object = static_cast<UObject*>(Container);

	if (IsValid(Object))
	{
		return Object;
	}

	return nullptr;
}

FText GetDebugPropertyDisplayText(const FProperty* InProperty, void* DataPtr)
{
	if (!InProperty)
	{
		return FText::GetEmpty();
	}

	if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(InProperty))
	{
		UObject* Value = nullptr;
		ObjectProperty->GetValue_InContainer(DataPtr, &Value);
		
		if (Value)
		{
			return FText::FromString(Value->GetPathName());
		}

		return LOCTEXT("Nullptr", "None");
	}
	else if (const FNameProperty* NameProperty = CastField<const FNameProperty>(InProperty))
	{
		FName Value = NAME_None;
		NameProperty->GetValue_InContainer(DataPtr, &Value);

		return FText::FromName(Value);
	}
	else if (const FStrProperty* StringProperty = CastField<const FStrProperty>(InProperty))
	{
		FString Value = TEXT("");
		StringProperty->GetValue_InContainer(DataPtr, &Value);

		return FText::Format(
			LOCTEXT("QuotedString", "\"{0}\""),
			FText::FromString(Value)
		);
	}
	else if (const FTextProperty* TextProperty = CastField<const FTextProperty>(InProperty))
	{
		FText Value = FText::GetEmpty();
		TextProperty->GetValue_InContainer(DataPtr, &Value);

		return FText::Format(
			LOCTEXT("QuotedString", "\"{0}\""),
			Value
		);
	}

	FString Value;
	InProperty->ExportTextItem_InContainer(Value, DataPtr, DataPtr, nullptr, PPF_None, nullptr);

	return FText::FromString(Value);
}

FReply HandleCallFunctionPressed(TWeakObjectPtr<UClass> ClassWeak, TWeakObjectPtr<UObject> ContainerWeak, TWeakObjectPtr<UFunction> FunctionWeak)
{
	UClass* Class = ClassWeak.Get();

	if (!Class)
	{
		return FReply::Handled();
	}

	UObject* Container = ContainerWeak.Get();

	if (!Container)
	{
		return FReply::Handled();
	}

	UFunction* Function = FunctionWeak.Get();

	if (!Function)
	{
		return FReply::Handled();
	}

	if (FProperty* ReturnProperty = GetReturnProperty(Function))
	{
		void* DataPtr = FMemory_Alloca_Aligned(Function->ParmsSize, Function->MinAlignment);
		void* ReturnPropertyPtr = (uint8*)DataPtr + ReturnProperty->GetOffset_ForUFunction();

		ReturnProperty->InitializeValue(ReturnPropertyPtr);

		Container->ProcessEvent(Function, DataPtr);

		const FText DisplayValue = GetDebugPropertyDisplayText(ReturnProperty, DataPtr);

		ReturnProperty->DestroyValue(ReturnPropertyPtr);

#if WITH_EDITOR
		FMessageDialog::Open(
			EAppMsgType::Ok,
			FText::Format(
				LOCTEXT("ReturnValueFormat", "{0} :: {1}\n\n{2}"),
				Class->GetDisplayNameText(),
				Function->GetDisplayNameText(),
				DisplayValue
			)
		);
#endif
	}
	else
	{
		Container->ProcessEvent(Function, nullptr);
	}

	return FReply::Handled();
}

TSharedPtr<SWidget> FPropertyValueFactory::Generate(FGenerateArgs Args) const
{
	if (UFunction* BaseFunction = Args.Field ? Cast<UFunction>(Args.Field->ToUObject()) : nullptr)
	{
		// Attempt to resolve property chain, but do not call functions to continue the chain.
		if (void* ContainerPtr = Args.Path.GetContainerPtr())
		{
			if (UObject* Container = GetUObjectFromContainer(ContainerPtr))
			{
				UClass* ContainerClass = Container->GetClass();

				if (const FProperty* LastProperty = Args.Path.GetLastProperty())
				{
					UClass* PropertyOwnerClass = LastProperty->GetOwnerClass();

					if (ContainerClass != PropertyOwnerClass && !ContainerClass->IsChildOf(PropertyOwnerClass))
					{
						return SNew(STextBlock)
							.Text(LOCTEXT("InvalidClass", "Invalid Class"))
							.IsEnabled(false);
					}

					if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(LastProperty))
					{
						UObject* NewContainer = nullptr;
						ObjectProperty->GetValue_InContainer(Container, &NewContainer);
						Container = NewContainer;
					}
					else
					{
						return SNew(STextBlock)
							.Text(LOCTEXT("InvalidProperty", "Invalid Property"))
							.IsEnabled(false);
					}

					if (!IsValid(Container))
					{
						return SNew(STextBlock)
							.Text(LOCTEXT("NoParentObject", "No Parent Object"))
							.IsEnabled(false);
					}
				}

				// Ensure the container actually has the function.
				if (UFunction* Function = Container->FindFunction(BaseFunction->GetFName()))
				{
					FProperty* ReturnProperty = GetReturnProperty(Function);

					if ((Function->NumParms == 0 && !ReturnProperty) || (Function->NumParms == 1 && ReturnProperty))
					{
						return SNew(SButton)
							.Text(LOCTEXT("CallFunction", "Call Function"))
							.OnClicked_Static(&HandleCallFunctionPressed, TWeakObjectPtr<UClass>(ContainerClass), TWeakObjectPtr(Container), TWeakObjectPtr<UFunction>(Function));
					}
				}
			}
		}

		return SNullWidget::NullWidget;
	}

	if (const FProperty* LastProperty = Args.Path.GetLastProperty())
	{
		const bool bIsVisible = LastProperty->HasAllPropertyFlags(CPF_BlueprintVisible);
		if (bIsVisible && LastProperty->ArrayDim == 1)
		{
			if (LastProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly))
			{
				Args.bCanEditValue = false;
			}

			if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(LastProperty))
			{
				// Class editors should consider parent classes.
				for (const UClass* Class = ObjectProperty->PropertyClass; !!Class; Class = Class->GetSuperClass())
				{
					if (Private::FPropertyViewerMapItem* FoundOnGenerate = Impl->RegisteredFieldEditor.Find(Class->GetStructPathName()))
					{
						check(FoundOnGenerate->OnGenerate.IsBound());
						return FoundOnGenerate->OnGenerate.Execute(Args);
					}
				}
			}
			else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(LastProperty))
			{
				if (Private::FPropertyViewerMapItem* FoundOnGenerate = Impl->RegisteredFieldEditor.Find(StructProperty->Struct->GetStructPathName()))
				{
					check(FoundOnGenerate->OnGenerate.IsBound());
					return FoundOnGenerate->OnGenerate.Execute(Args);
				}
			}
			else
			{
				if (Private::FPropertyViewerMapItem* FoundOnGenerate = Impl->RegisteredFieldClassEditor.Find(LastProperty->GetClass()))
				{
					check(FoundOnGenerate->OnGenerate.IsBound());
					return FoundOnGenerate->OnGenerate.Execute(Args);
				}
				for (auto& ClassEditor : Impl->RegisteredFieldClassEditor)
				{
					if (LastProperty->GetClass()->IsChildOf(ClassEditor.Key))
					{
						check(ClassEditor.Value.OnGenerate.IsBound());
						return ClassEditor.Value.OnGenerate.Execute(Args);
					}
				}
			}
		}
	}

	return TSharedPtr<SWidget>();
}


TSharedPtr<SWidget> FPropertyValueFactory::GenerateDefault(FGenerateArgs Args) const
{
	if (const FProperty* LastProperty = Args.Path.GetLastProperty())
	{
		const bool bIsVisible = LastProperty->HasAllPropertyFlags(CPF_BlueprintVisible);
		if (bIsVisible)
		{
			if (LastProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly))
			{
				Args.bCanEditValue = false;
			}

			return SDefaultPropertyValue::CreateInstance(Args);
		}
	}
	return TSharedPtr<SWidget>();
}


bool FPropertyValueFactory::HasCustomPropertyValue(const FFieldClass* FieldClass) const
{
	check(FieldClass);
	return Impl->RegisteredFieldClassEditor.Find(FieldClass) != nullptr;
}


bool FPropertyValueFactory::HasCustomPropertyValue(const UStruct* Struct) const
{
	check(Struct);

	// Class editors should consider parent classes.
	if (const UClass* Class = Cast<UClass>(Struct))
	{
		for (const UClass* Check = Class; !!Check; Check = Check->GetSuperClass())
		{
			if (Impl->RegisteredFieldEditor.Find(Check->GetStructPathName()) != nullptr)
			{
				return true;
			}
		}

		return false;
	}

	return Impl->RegisteredFieldEditor.Find(Struct->GetStructPathName()) != nullptr;
}


FPropertyValueFactory::FHandle FPropertyValueFactory::Register(const FFieldClass* FieldClass, FOnGenerate OnGenerateFieldEditor)
{
	if (!FieldClass || !OnGenerateFieldEditor.IsBound())
	{
		return FHandle();
	}

	FPropertyValueFactory::FHandle Result = MakeHandle();
	Private::FPropertyViewerMapItem Item;
	Item.OnGenerate = MoveTemp(OnGenerateFieldEditor);
	Item.Handle = Result;

	Impl->RegisteredFieldClassEditor.FindOrAdd(FieldClass) = MoveTemp(Item);

	return Result;
}


FPropertyValueFactory::FHandle FPropertyValueFactory::Register(const UStruct* Struct, FOnGenerate OnGenerateFieldEditor)
{
	if (!Struct || !OnGenerateFieldEditor.IsBound())
	{
		return FHandle();
	}

	FPropertyValueFactory::FHandle Result = MakeHandle();
	Private::FPropertyViewerMapItem Item;
	Item.OnGenerate = MoveTemp(OnGenerateFieldEditor);
	Item.Handle = Result;

	Impl->RegisteredFieldEditor.FindOrAdd(Struct->GetStructPathName()) = MoveTemp(Item);
	return Result;
}


void FPropertyValueFactory::Unregister(FHandle Handle)
{
	for (auto It = Impl->RegisteredFieldEditor.CreateIterator(); It; ++It)
	{
		if (It->Value.Handle == Handle)
		{
			It.RemoveCurrent();
			return;
		}
	}
	for (auto It = Impl->RegisteredFieldClassEditor.CreateIterator(); It; ++It)
	{
		if (It->Value.Handle == Handle)
		{
			It.RemoveCurrent();
			return;
		}
	}
}


FPropertyValueFactory::FHandle  FPropertyValueFactory::MakeHandle()
{
	static int32 Generator = 0;
	++Generator;
	FHandle Result;
	Result.Id = Generator;
	return Result;
}

} //namespace

#undef LOCTEXT_NAMESPACE
