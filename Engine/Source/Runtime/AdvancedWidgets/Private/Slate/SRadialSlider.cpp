// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRadialSlider.h"

#include "Brushes/SlateColorBrush.h"
#include "Fonts/FontMeasure.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElements.h"

#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleWidgets.h"
#endif

SRadialSlider::SRadialSlider()
{
#if WITH_ACCESSIBILITY
	AccessibleBehavior = EAccessibleBehavior::Summary;
	bCanChildrenBeAccessible = false;
#endif
}

SRadialSlider::~SRadialSlider() = default;

void SRadialSlider::Construct( const SRadialSlider::FArguments& InDeclaration )
{
	check(InDeclaration._Style);
	Style = InDeclaration._Style;

	bMouseUsesStep = InDeclaration._MouseUsesStep;
	bRequiresControllerLock = InDeclaration._RequiresControllerLock;
	LockedAttribute = InDeclaration._Locked;
	StepSize = InDeclaration._StepSize;
	ValueAttribute = InDeclaration._Value;
	bUseCustomDefaultValue = InDeclaration._bUseCustomDefaultValue;
	CustomDefaultValue = InDeclaration._CustomDefaultValue;
	SliderRange = InDeclaration._SliderRange;
	SliderHandleStartAngle = InDeclaration._SliderHandleStartAngle;
	SliderHandleEndAngle = InDeclaration._SliderHandleEndAngle;
	AngularOffset = InDeclaration._AngularOffset;
	HandStartEndRatio = InDeclaration._HandStartEndRatio;
	SliderBarColor = InDeclaration._SliderBarColor;
	SliderProgressColor = InDeclaration._SliderProgressColor;
	SliderHandleColor = InDeclaration._SliderHandleColor;
	CenterBackgroundColor = InDeclaration._CenterBackgroundColor;
	CenterBackgroundBrush = InDeclaration._CenterBackgroundBrush;
	bIsFocusable = InDeclaration._IsFocusable;
	bUseVerticalDrag = InDeclaration._UseVerticalDrag;
	bShowSliderHandle = InDeclaration._ShowSliderHandle;
	bShowSliderHand = InDeclaration._ShowSliderHand;
	OnMouseCaptureBegin = InDeclaration._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InDeclaration._OnMouseCaptureEnd;
	OnControllerCaptureBegin = InDeclaration._OnControllerCaptureBegin;
	OnControllerCaptureEnd = InDeclaration._OnControllerCaptureEnd;
	OnValueChanged = InDeclaration._OnValueChanged;

	bControllerInputCaptured = false;
	bIsUsingFineTune = false;
	FineTuneKey = EKeys::LeftShift;
}

int32 SRadialSlider::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const float AllottedWidth = AllottedGeometry.GetLocalSize().X;
	const float AllottedHeight = AllottedGeometry.GetLocalSize().Y;
	FGeometry SliderGeometry = AllottedGeometry;

	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	// Draw radial slider bar
	const FVector2D HandleSize = GetThumbImage()->ImageSize;
	const FVector2D HalfHandleSize = 0.5f * HandleSize;
	const float SliderRadius = FMath::Min(AllottedWidth, AllottedHeight) * 0.5f - HalfHandleSize.Y;
	const FVector2D StartPoint(0.0f, SliderRadius);

	const float MidPointAngle = FMath::Lerp(SliderHandleStartAngle, SliderHandleEndAngle, GetNormalizedSliderHandlePosition());

	const FVector2D HandleLocation = StartPoint.GetRotated(MidPointAngle + AngularOffset);

	TArray<FVector2D> SliderBarPoints;
	TArray<FVector2D> ProgressBarPoints;
	
	const float NormalizedCustomDefaultValue = bUseCustomDefaultValue.Get() ? GetNormalizedValue(CustomDefaultValue.Get()) : GetMinValue();
	const bool bIsCCW = GetNormalizedSliderHandlePosition() < NormalizedCustomDefaultValue;

	const float CustomDefaultValueAngle = FMath::Lerp(SliderHandleStartAngle, SliderHandleEndAngle, NormalizedCustomDefaultValue);
	const FVector2D CustomDefaultValueLocation = StartPoint.GetRotated(CustomDefaultValueAngle + AngularOffset);
	   	  
	ProgressBarPoints.Emplace(bIsCCW ? HandleLocation : CustomDefaultValueLocation);

	static const int32 CircleResolution = 100;
	for (int32 i = 0; i <= CircleResolution; i++)
	{
		const float CircleAnglePercent = float(i) / float(CircleResolution);
		const float CurrentPointAngle = FMath::Lerp(SliderHandleStartAngle, SliderHandleEndAngle, CircleAnglePercent);

		const FVector2D NewCirclePoint = StartPoint.GetRotated(CurrentPointAngle + AngularOffset);
		
		const bool bIsUniquePoint(i != CircleResolution);
		AddSliderPointToArray(SliderBarPoints, bIsUniquePoint, NewCirclePoint);

		// draw completed progress bar, given directionality
		const bool bShouldDrawProgressbar = (
			(bIsCCW && CircleAnglePercent >= GetNormalizedSliderHandlePosition() && CircleAnglePercent <= NormalizedCustomDefaultValue) ||
			(!bIsCCW && CircleAnglePercent <= GetNormalizedSliderHandlePosition() && CircleAnglePercent >= NormalizedCustomDefaultValue)
		);
		
		if (bShouldDrawProgressbar)
		{
			AddSliderPointToArray(ProgressBarPoints, bIsUniquePoint, NewCirclePoint);
		}
	}

	ProgressBarPoints.AddUnique(bIsCCW ? CustomDefaultValueLocation : HandleLocation);

	const FVector2D SliderMidPoint(AllottedGeometry.GetLocalSize() * 0.5f);
	const FVector2D SliderDiameter(SliderRadius * 2.0f);
	
	// combine bar color alpha with widget alpha
	FLinearColor SliderBarColorWithAlpha = SliderBarColor.Get().GetColor(InWidgetStyle);
	SliderBarColorWithAlpha.A *= GetRenderOpacity();
	FLinearColor ProgressBarColorWithAlpha = SliderProgressColor.Get().GetColor(InWidgetStyle);
	ProgressBarColorWithAlpha.A *= GetRenderOpacity();

	float BarThickness;
	// For backwards compat, if Thickness isn't set, use Style->BarThickness
	if (Thickness.Get().IsSet())
	{
		// Compensate for the widget size when passing screenspace thickness to MakeLines  
		// TODO: Compensate thickness based on zoom level
		float ScaleFactor = FMath::Min(AllottedWidth, AllottedHeight) / 100.0f;
		BarThickness = Thickness.Get().GetValue() * ScaleFactor;
	}
	else
	{
		BarThickness = Style->BarThickness;
	}

	// draw slider bar
	FSlateDrawElement::MakeLines
	(
		OutDrawElements,
		LayerId,
		SliderGeometry.ToPaintGeometry(SliderDiameter, FSlateLayoutTransform(SliderMidPoint)),
		SliderBarPoints,
		DrawEffects,
		SliderBarColorWithAlpha,
		true,
		BarThickness
	);

	// draw completed progress bar
	FSlateDrawElement::MakeLines
	(
		OutDrawElements,
		LayerId,
		SliderGeometry.ToPaintGeometry(SliderDiameter, FSlateLayoutTransform(SliderMidPoint)),
		ProgressBarPoints,
		DrawEffects,
		ProgressBarColorWithAlpha,
		true,
		BarThickness 
	);

	// draw center background circle
	FLinearColor CenterBackgroundColorWithAlpha = CenterBackgroundColor.Get().GetColor(InWidgetStyle);
	CenterBackgroundColorWithAlpha.A *= GetRenderOpacity();

	FSlateDrawElement::MakeBox
	(
		OutDrawElements, 
		LayerId, 
		SliderGeometry.ToPaintGeometry(SliderDiameter, FSlateLayoutTransform(SliderMidPoint - SliderRadius)),
		&CenterBackgroundBrush,
		DrawEffects, 
		CenterBackgroundColorWithAlpha
	);
	++LayerId;

	// Draw hand from middle to handle 
	if (bShowSliderHand)
	{
		const FVector2D HandStart(0.0f, SliderRadius * HandStartEndRatio.X);
		const FVector2D HandEnd(0.0f, SliderRadius * HandStartEndRatio.Y);

		const FVector2D HandStartRotated = HandStart.GetRotated(MidPointAngle + AngularOffset);
		const FVector2D HandEndRotated = HandEnd.GetRotated(MidPointAngle + AngularOffset);

		FSlateDrawElement::MakeLines
		(
			OutDrawElements,
			LayerId,
			SliderGeometry.ToPaintGeometry(SliderDiameter, FSlateLayoutTransform(SliderMidPoint)),
			TArray<FVector2D>{HandStartRotated, HandEndRotated},
			DrawEffects,
			SliderBarColorWithAlpha,
			true,
			BarThickness 
		);
		++LayerId;
	}
	
	TArray<float> DrawnTagValues;
	for (int32 TagIndex = 0; TagIndex < ValueTags.Num(); TagIndex++)
	{
		const float CandidateValue = ValueTags[TagIndex];

		if (CandidateValue < GetMinValue() || CandidateValue > GetMaxValue())
		{
			// Do not draw tags for values that are outside of the slider range
			continue;
		}
		else if (DrawnTagValues.Find(CandidateValue) != INDEX_NONE) // Ignore duplicated tags
		{
			continue;
		}
		else
		{
			DrawnTagValues.Emplace(CandidateValue);
		}

		const float NormalizedCandidateTagPosition = GetNormalizedValue(CandidateValue);
		static const float TextSpacing(5.0f);

		const float TagAngle = FMath::Lerp(SliderHandleStartAngle, SliderHandleEndAngle, NormalizedCandidateTagPosition);

		FVector2D LineStartLocation(FVector2D(StartPoint.X, StartPoint.Y));
		FVector2D LineEndLocation(FVector2D(StartPoint.X, StartPoint.Y + TextSpacing));
		LineStartLocation = LineStartLocation.GetRotated(TagAngle + AngularOffset);
		LineEndLocation = LineEndLocation.GetRotated(TagAngle + AngularOffset);

		TArray<FVector2D> TagLinePoints;
		TagLinePoints.Emplace(LineStartLocation);
		TagLinePoints.Emplace(LineEndLocation);

		const bool bIsTagLineInProgressBarRange = (
			(bIsCCW && NormalizedCandidateTagPosition >= GetNormalizedSliderHandlePosition() && NormalizedCandidateTagPosition <= NormalizedCustomDefaultValue) ||
			(!bIsCCW && NormalizedCandidateTagPosition <= GetNormalizedSliderHandlePosition() && NormalizedCandidateTagPosition >= NormalizedCustomDefaultValue)
			);

		FLinearColor TagLineColor = bIsTagLineInProgressBarRange ? ProgressBarColorWithAlpha : SliderBarColorWithAlpha;

		// draw tag line
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			SliderGeometry.ToPaintGeometry(SliderDiameter, FSlateLayoutTransform(SliderMidPoint)),
			TagLinePoints,
			DrawEffects,
			TagLineColor,
			true,
			BarThickness
		);

		const FText TextToWrite = FText::FromString(FString::SanitizeFloat(CandidateValue));
		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		FSlateFontInfo MyFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);
		const FVector2D TextDrawSize = FontMeasureService->Measure(TextToWrite, MyFont);

		FVector2D TextLocation(FVector2D(StartPoint.X, StartPoint.Y + TextSpacing + 10.0f));
		TextLocation = TextLocation.GetRotated(TagAngle + AngularOffset);

		const FVector2D TextTopLeftPoint = TextLocation + (AllottedGeometry.GetLocalSize() * 0.5f) - TextDrawSize * 0.5f;

		// draw tag value
		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId,
			SliderGeometry.ToPaintGeometry(GetThumbImage()->ImageSize, FSlateLayoutTransform(TextTopLeftPoint)),
			TextToWrite,
			MyFont,
			DrawEffects,
			FLinearColor::White
		);
	}

	// Draw slider handle (thumb)
	if (bShowSliderHandle)
	{
		const FVector2D HandleTopLeftPoint = HandleLocation + (AllottedGeometry.GetLocalSize() * 0.5f) - HalfHandleSize;

		auto ThumbImage = GetThumbImage();
		FSlateDrawElement::MakeRotatedBox(
			OutDrawElements,
			LayerId,
			SliderGeometry.ToPaintGeometry(GetThumbImage()->ImageSize, FSlateLayoutTransform(HandleTopLeftPoint)),
			ThumbImage,
			DrawEffects,
			(180.0f + MidPointAngle + AngularOffset) * (PI / 180.0f),
			HalfHandleSize,
			FSlateDrawElement::RelativeToElement,
			ThumbImage->GetTint(InWidgetStyle) * SliderHandleColor.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);
	}

	++LayerId;
		
	return LayerId;
}

void SRadialSlider::AddSliderPointToArray(TArray<FVector2D>& SliderPoints, const bool bIsUnique, const FVector2D& SliderPoint) const
{
	if (bIsUnique)
	{
		SliderPoints.AddUnique(SliderPoint);
	}
	else
	{
		SliderPoints.Emplace(SliderPoint);
	}
}

FVector2D SRadialSlider::ComputeDesiredSize( float ) const
{
	return FVector2D(200.0f);
}

bool SRadialSlider::IsLocked() const
{
	return LockedAttribute.Get();
}

bool SRadialSlider::IsInteractable() const
{
	return IsEnabled() && !IsLocked() && SupportsKeyboardFocus();
}

bool SRadialSlider::SupportsKeyboardFocus() const
{
	return bIsFocusable;
}

void SRadialSlider::ResetControllerState()
{
	if (bControllerInputCaptured)
	{
		OnControllerCaptureEnd.ExecuteIfBound();
		bControllerInputCaptured = false;
	}
}

FNavigationReply SRadialSlider::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
{
	if (bControllerInputCaptured || !bRequiresControllerLock)
	{
		FNavigationReply Reply = FNavigationReply::Escape();

		float NewValue = ValueAttribute.Get();
		if (InNavigationEvent.GetNavigationType() == EUINavigation::Left)
		{
			NewValue -= StepSize.Get();
			Reply = FNavigationReply::Stop();
		}
		else if (InNavigationEvent.GetNavigationType() == EUINavigation::Right)
		{
			NewValue += StepSize.Get();
			Reply = FNavigationReply::Stop();
		}

		if (ValueAttribute.Get() != NewValue)
		{
			CommitValue(FMath::Clamp(NewValue, GetMinValue(), GetMaxValue()));
			return Reply;
		}
	}

	return SLeafWidget::OnNavigation(MyGeometry, InNavigationEvent);
}

FReply SRadialSlider::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();
	
	if (InKeyEvent.GetKey() == FineTuneKey)
	{
		bIsUsingFineTune = true;
	}

	if (IsInteractable())
	{
		// The controller's bottom face button must be pressed once to begin manipulating the slider's value.
		// Navigation away from the widget is prevented until the button has been pressed again or focus is lost.
		// The value can be manipulated by using the game pad's directional arrows ( relative to slider orientation ).
		if (FSlateApplication::Get().GetNavigationActionFromKey(InKeyEvent) == EUINavigationAction::Accept && bRequiresControllerLock)
		{
			if (bControllerInputCaptured == false)
			{
				// Begin capturing controller input and allow user to modify the slider's value.
				bControllerInputCaptured = true;
				OnControllerCaptureBegin.ExecuteIfBound();
				Reply = FReply::Handled();
			}
			else
			{
				ResetControllerState();
				Reply = FReply::Handled();
			}
		}
		else
		{
			Reply = SLeafWidget::OnKeyDown(MyGeometry, InKeyEvent);
		}
	}
	else
	{
		Reply = SLeafWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

	return Reply;
}

FReply SRadialSlider::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FReply Reply = FReply::Unhandled();

	bIsUsingFineTune = false;

	if (bControllerInputCaptured)
	{
		Reply = FReply::Handled();
	}
	return Reply;
}

void SRadialSlider::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	if (bControllerInputCaptured)
	{
		// Commit and reset state
		CommitValue(ValueAttribute.Get());
		ResetControllerState();
	}
}

void SRadialSlider::OnInputStarted(const FGeometry& MyGeometry, const FVector2D& InputAbsolutePosition)
{
	AbsoluteInputAngle = GetAngleFromPosition(MyGeometry, InputAbsolutePosition);
	PreviousAbsolutePosition = InputAbsolutePosition;
}

FReply SRadialSlider::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && !IsLocked())
	{
		CachedCursor = GetCursor().Get(EMouseCursor::Default);
		OnInputStarted(MyGeometry, MouseEvent.GetLastScreenSpacePosition());

		OnMouseCaptureBegin.ExecuteIfBound();
		CommitValue(PositionToValue(MyGeometry, MouseEvent.GetLastScreenSpacePosition()));
		
		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();
		
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply SRadialSlider::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && HasMouseCaptureByUser(MouseEvent.GetUserIndex(), MouseEvent.GetPointerIndex()))
	{
		SetCursor(CachedCursor);
		OnMouseCaptureEnd.ExecuteIfBound();
		
		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();
		
		return FReply::Handled().ReleaseMouseCapture();	
	}

	return FReply::Unhandled();
}

FReply SRadialSlider::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (HasMouseCaptureByUser(MouseEvent.GetUserIndex(), MouseEvent.GetPointerIndex()) && !IsLocked())
	{
		SetCursor(EMouseCursor::GrabHandClosed);
		CommitValue(PositionToValue(MyGeometry, MouseEvent.GetLastScreenSpacePosition()));
		
		// Release capture for controller/keyboard when switching to mouse
		ResetControllerState();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SRadialSlider::OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (!IsLocked())
	{
		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();

		PressedScreenSpaceTouchDownPosition = InTouchEvent.GetScreenSpacePosition();
		OnInputStarted(MyGeometry, PressedScreenSpaceTouchDownPosition);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SRadialSlider::OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (HasMouseCaptureByUser(InTouchEvent.GetUserIndex(), InTouchEvent.GetPointerIndex()))
	{
		CommitValue(PositionToValue(MyGeometry, InTouchEvent.GetScreenSpacePosition()));

		// Release capture for controller/keyboard when switching to mouse
		ResetControllerState();

		return FReply::Handled();
	}
	else if (!HasMouseCapture())
	{
		if (FSlateApplication::Get().HasTraveledFarEnoughToTriggerDrag(InTouchEvent, PressedScreenSpaceTouchDownPosition, EOrientation::Orient_Horizontal))
		{
			CachedCursor = GetCursor().Get(EMouseCursor::Default);
			OnMouseCaptureBegin.ExecuteIfBound();

			CommitValue(PositionToValue(MyGeometry, InTouchEvent.GetScreenSpacePosition()));

			// Release capture for controller/keyboard when switching to mouse
			ResetControllerState();

			return FReply::Handled().CaptureMouse(SharedThis(this));
		}
	}

	return FReply::Unhandled();
}

FReply SRadialSlider::OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	if (HasMouseCaptureByUser(InTouchEvent.GetUserIndex(), InTouchEvent.GetPointerIndex()))
	{
		SetCursor(CachedCursor);
		OnMouseCaptureEnd.ExecuteIfBound();

		CommitValue(PositionToValue(MyGeometry, InTouchEvent.GetScreenSpacePosition()));

		// Release capture for controller/keyboard when switching to mouse.
		ResetControllerState();

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

void SRadialSlider::CommitValue(float NewValue)
{
	ValueAttribute.Set(NewValue);
	Invalidate(EInvalidateWidgetReason::Paint);
	OnValueChanged.ExecuteIfBound(FMath::Clamp(NewValue, GetMinValue(), GetMaxValue()));
}

float SRadialSlider::GetAngleFromPosition(const FGeometry& MyGeometry, const FVector2D& AbsolutePosition)
{
	const FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(AbsolutePosition) - MyGeometry.GetLocalSize() * 0.5f;
	const FVector2D InputDirection = LocalPosition.GetSafeNormal().GetRotated(90.0f - AngularOffset);
	const float Angle = 180.0f + (180.0f / PI * FMath::Atan2(InputDirection.Y, InputDirection.X));

	return Angle;
}

float SRadialSlider::PositionToValue(const FGeometry& MyGeometry, const FVector2D& AbsolutePosition)
{
	float NewValue;
	float CurveTime; 
	const FRichCurve* SliderRangeCurve = SliderRange.GetRichCurveConst();
	float CurveMinTime;
	float CurveMaxTime;
	SliderRangeCurve->GetTimeRange(CurveMinTime, CurveMaxTime);

	if (bUseVerticalDrag)
	{
		const float VerticalDragMouseSpeed = bIsUsingFineTune ? VerticalDragMouseSpeedFineTune : VerticalDragMouseSpeedNormal;
		float ValueDelta = (float)(AbsolutePosition.Y - PreviousAbsolutePosition.Y) / VerticalDragPixelDelta * VerticalDragMouseSpeed;
		PreviousAbsolutePosition = AbsolutePosition;
		CurveTime = FMath::Clamp(ValueAttribute.Get() - ValueDelta, 0.0f, 1.0f);
	}
	else
	{
		const float OldAngle = GetAngleFromPosition(MyGeometry, PreviousAbsolutePosition);
		const float NewAngle = GetAngleFromPosition(MyGeometry, AbsolutePosition);

		float WrappedDelta = FMath::Abs(FMath::Fmod((NewAngle - OldAngle), 360.0f));
		float AngularDelta = WrappedDelta > 180.0f ? 360.0f - WrappedDelta : WrappedDelta;

		int32 Sign = (NewAngle - OldAngle >= 0.0f && NewAngle - OldAngle <= 180.0f) || (NewAngle - OldAngle <= -180 && NewAngle - OldAngle >= -360) ? 1 : -1;
		AngularDelta *= Sign;

		PreviousAbsolutePosition = AbsolutePosition;
		AbsoluteInputAngle += AngularDelta;
		CurveTime = FMath::GetMappedRangeValueClamped(FVector2D(SliderHandleStartAngle, SliderHandleEndAngle), FVector2D(CurveMinTime, CurveMaxTime), AbsoluteInputAngle);
	}

	NewValue = SliderRangeCurve->Eval(CurveTime);
	if (bMouseUsesStep)
	{
		const float SteppedValue = FMath::RoundToInt(NewValue / StepSize.Get()) * StepSize.Get();
		NewValue = SteppedValue;
	}

	return NewValue;
}

const FSlateBrush* SRadialSlider::GetBarImage() const
{
	if (!IsEnabled() || LockedAttribute.Get())
	{
		return &Style->DisabledBarImage;
	}
	else if (IsHovered())
	{
		return &Style->HoveredBarImage;
	}
	else
	{
		return &Style->NormalBarImage;
	}
}

const FSlateBrush* SRadialSlider::GetThumbImage() const
{
	if (!IsEnabled() || LockedAttribute.Get())
	{
		return &Style->DisabledThumbImage;
	}
	else if (IsHovered())
	{
		return &Style->HoveredThumbImage;
	}
	else
	{
		return &Style->NormalThumbImage;
	}
}

float SRadialSlider::GetMinValue() const
{
	float MinValue = 0.0f;

	const FRichCurve* SliderRangeCurve = SliderRange.GetRichCurveConst();

	if (SliderRangeCurve->GetNumKeys() > 0)
	{
		MinValue = SliderRangeCurve->GetFirstKey().Value;
	}
	
	return MinValue;
}

float SRadialSlider::GetMaxValue() const
{
	float MaxValue = 0.0f;

	const FRichCurve* SliderRangeCurve = SliderRange.GetRichCurveConst();

	if (SliderRangeCurve->GetNumKeys() > 0)
	{
		MaxValue = SliderRangeCurve->GetLastKey().Value;
	}

	return MaxValue;
}

float SRadialSlider::GetValue() const
{
	return ValueAttribute.Get();
}

bool SRadialSlider::GetUseCustomDefaultValue() const
{
	return bUseCustomDefaultValue.Get();
}

float SRadialSlider::GetCustomDefaultValue() const
{
	return CustomDefaultValue.Get();
}

float SRadialSlider::GetNormalizedValue(float RawValue) const
{
	float NormalizedValue(0.0f);	

	const TArray<FRichCurveKey>& SliderRangeCurveKeys = SliderRange.GetRichCurveConst()->GetConstRefOfKeys();
	
	if (SliderRangeCurveKeys.Num() > 0)
	{
		float SliderPercentMin(0.0f);
		float SliderPercentMax(1.0f);

		float SampleValueMin(GetMinValue());
		float SampleValueMax(GetMaxValue());

		const float MinCurveTime = SliderRangeCurveKeys[0].Time;
		const float MaxCurveTime = SliderRangeCurveKeys[SliderRangeCurveKeys.Num() -1].Time;

		for (int32 i = 0; i < SliderRangeCurveKeys.Num(); i++)
		{
			if (RawValue < SliderRangeCurveKeys[i].Value)
			{
				SliderPercentMax = (SliderRangeCurveKeys[i].Time - MinCurveTime) / (MaxCurveTime - MinCurveTime);
				SampleValueMax = SliderRangeCurveKeys[i].Value;
				break;
			}

			SliderPercentMin = (SliderRangeCurveKeys[i].Time - MinCurveTime) / (MaxCurveTime - MinCurveTime);
			SampleValueMin = SliderRangeCurveKeys[i].Value;
		}

		NormalizedValue = FMath::GetMappedRangeValueClamped(FVector2f(SampleValueMin, SampleValueMax), FVector2f(SliderPercentMin, SliderPercentMax), RawValue);
	}

	return NormalizedValue;
}

float SRadialSlider::GetNormalizedSliderHandlePosition() const
{
	return GetNormalizedValue(ValueAttribute.Get());
}

void SRadialSlider::SetValue(const TAttribute<float>& InValueAttribute)
{
	SetAttribute(ValueAttribute, InValueAttribute, EInvalidateWidgetReason::Paint);
}

void SRadialSlider::SetUseCustomDefaultValue(const TAttribute<bool>& InValueAttribute)
{
	SetAttribute(bUseCustomDefaultValue, InValueAttribute, EInvalidateWidgetReason::Paint);
}

void SRadialSlider::SetCustomDefaultValue(const TAttribute<float>& InValueAttribute)
{
	SetAttribute(CustomDefaultValue, InValueAttribute, EInvalidateWidgetReason::Paint);
}

void SRadialSlider::SetSliderHandleStartAngleAndSliderHandleEndAngle(float InSliderHandleStartAngle, float InSliderHandleEndAngle)
{
	SliderHandleStartAngle = InSliderHandleStartAngle;
	SliderHandleEndAngle = InSliderHandleEndAngle;
	if (SliderHandleStartAngle > SliderHandleEndAngle)
	{
		SliderHandleEndAngle = SliderHandleStartAngle;
	}
}

void SRadialSlider::SetHandStartEndRatio(FVector2D InHandStartEndRatio)
{
	const FVector2D ClampedRatio = InHandStartEndRatio.ClampAxes(0.0f, 1.0f);
	if (ClampedRatio.X > ClampedRatio.Y)
	{
		HandStartEndRatio = FVector2D(ClampedRatio.X);
	}
	else
	{
		HandStartEndRatio = ClampedRatio;
	}
}

void SRadialSlider::SetLocked(const TAttribute<bool>& InLocked)
{
	SetAttribute(LockedAttribute, InLocked, EInvalidateWidgetReason::Paint);
}

void SRadialSlider::SetSliderBarColor(FSlateColor InSliderBarColor)
{
	SetAttribute(SliderBarColor, TAttribute<FSlateColor>(InSliderBarColor), EInvalidateWidgetReason::Paint);
}

void SRadialSlider::SetSliderProgressColor(FSlateColor InSliderProgressColor)
{
	SetAttribute(SliderProgressColor, TAttribute<FSlateColor>(InSliderProgressColor), EInvalidateWidgetReason::Paint);
}

void SRadialSlider::SetSliderHandleColor(FSlateColor InSliderHandleColor)
{
	SetAttribute(SliderHandleColor, TAttribute<FSlateColor>(InSliderHandleColor), EInvalidateWidgetReason::Paint);
}

void SRadialSlider::SetCenterBackgroundColor(FSlateColor InCenterBackgroundColor)
{
	SetAttribute(CenterBackgroundColor, TAttribute<FSlateColor>(InCenterBackgroundColor), EInvalidateWidgetReason::Paint);
}

float SRadialSlider::GetStepSize() const
{
	return StepSize.Get();
}

void SRadialSlider::SetStepSize(const TAttribute<float>& InStepSize)
{
	StepSize = InStepSize;
}

void SRadialSlider::SetMouseUsesStep(bool MouseUsesStep)
{
	bMouseUsesStep = MouseUsesStep;
}

void SRadialSlider::SetRequiresControllerLock(bool RequiresControllerLock)
{
	bRequiresControllerLock = RequiresControllerLock;
}

void SRadialSlider::SetUseVerticalDrag(bool UseVerticalDrag)
{
	bUseVerticalDrag = UseVerticalDrag;
}

void SRadialSlider::SetShowSliderHandle(bool ShowSliderHandle)
{
	bShowSliderHandle = ShowSliderHandle;
}

void SRadialSlider::SetShowSliderHand(bool ShowSliderHand)
{
	bShowSliderHand = ShowSliderHand;
}

void SRadialSlider::SetThickness(const float InThickness)
{
	SetAttribute(Thickness, TAttribute<TOptional<float>>(InThickness), EInvalidateWidgetReason::Paint);
}

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> SRadialSlider::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleSlider(SharedThis(this)));
}
#endif
