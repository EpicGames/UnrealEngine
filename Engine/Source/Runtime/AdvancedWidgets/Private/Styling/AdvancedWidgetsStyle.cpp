// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/AdvancedWidgetsStyle.h"

#include "Framework/PropertyViewer/FieldIconFinder.h"
#include "Math/ColorList.h"
#include "Styling/AppStyle.h"
#include "Styling/ColorGradingSpinBoxStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "UObject/Class.h"


#define BORDER_BRUSH( RelativePath, ... ) FSlateBorderBrush( Instance->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( Instance->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( Instance->RootToContentDir(RelativePath, TEXT(".svg") ), __VA_ARGS__)

namespace UE::AdvancedWidgets
{

TUniquePtr<FSlateStyleSet> FAdvancedWidgetsStyle::Instance;
::UE::PropertyViewer::FFieldColorSettings FAdvancedWidgetsStyle::ColorSettings;


const ISlateStyle& FAdvancedWidgetsStyle::Get()
{
	return *(Instance.Get());
}

void FAdvancedWidgetsStyle::Create()
{
	Instance = MakeUnique<FSlateStyleSet>("AdvancedWidgets");
	Instance->SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
	Instance->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Instance->Set("PropertyValue.SpinBox", FSpinBoxStyle(FAppStyle::GetWidgetStyle<FSpinBoxStyle>("SpinBox"))
		.SetTextPadding(FMargin(0.f))
		.SetBackgroundBrush(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0.f, 0.f, 0.f, 3.f / 16.f), FSlateColor::UseSubduedForeground()))
		.SetHoveredBackgroundBrush(FSlateNoResource())
		.SetInactiveFillBrush(FSlateNoResource())
		.SetActiveFillBrush(FSlateNoResource())
		.SetForegroundColor(FSlateColor::UseSubduedForeground())
		.SetArrowsImage(FSlateNoResource())
	);

	FButtonStyle Button = FAppStyle::GetWidgetStyle<FButtonStyle>("Button");
	Instance->Set("PropertyValue.ComboButton", FComboButtonStyle(FAppStyle::GetWidgetStyle<FComboButtonStyle>("ComboButton"))
		.SetButtonStyle(
			Button.SetNormal(FSlateNoResource())
			.SetNormalForeground(FSlateColor::UseSubduedForeground())
			.SetDisabledForeground(FSlateColor::UseSubduedForeground())
			.SetNormalPadding(FMargin(0.f))
			.SetPressedPadding(FMargin(0.f))
		)
		.SetMenuBorderPadding(FMargin(0.0f))
		.SetDownArrowPadding(FMargin(0.0f))
		);

	Instance->Set("ColorGradingWheel.Cross", new IMAGE_BRUSH_SVG("Starship/Common/color-grading-cross", FVector2f(336.f, 336.f)));
	Instance->Set("ColorGradingWheel.Selector", new IMAGE_BRUSH_SVG("Starship/Common/color-grading-selector", FVector2f(16.f, 16.f)));

	const FSlateBrush* NoBrush = FCoreStyle::Get().GetBrush("NoBrush");
	check(NoBrush);

	{
		FSpinBoxStyle NumericEntrySpinBoxStyle = FCoreStyle::Get().GetWidgetStyle<FSpinBoxStyle>("NumericEntrySpinBox");
		NumericEntrySpinBoxStyle.ActiveFillBrush = *NoBrush;
		NumericEntrySpinBoxStyle.HoveredFillBrush = *NoBrush;
		NumericEntrySpinBoxStyle.InactiveFillBrush = *NoBrush;

		Instance->Set("ColorGradingComponentViewer.NumericEntry", NumericEntrySpinBoxStyle);
		Instance->Set("ColorGradingComponentViewer.NumericEntry.TextBox", FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"));

		Instance->Set("ColorGradingSlider.Border", new FSlateRoundedBoxBrush(
			FStyleColors::Input, CoreStyleConstants::InputFocusRadius,
			FStyleColors::Secondary, CoreStyleConstants::InputFocusThickness)
		);
	}

	{
		FSpinBoxStyle DefaultSpinBox = FCoreStyle::Get().GetWidgetStyle<FSpinBoxStyle>("SpinBox");

		Instance->Set("ColorGradingSpinBox", FColorGradingSpinBoxStyle()
			.SetBorderBrush(FSlateRoundedBoxBrush(FStyleColors::Transparent, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness))
			.SetActiveBorderBrush(FSlateRoundedBoxBrush(FStyleColors::Transparent, CoreStyleConstants::InputFocusRadius, FStyleColors::Primary, CoreStyleConstants::InputFocusThickness))
			.SetHoveredBorderBrush(FSlateRoundedBoxBrush(FStyleColors::Transparent, CoreStyleConstants::InputFocusRadius, FStyleColors::Hover, CoreStyleConstants::InputFocusThickness))
			.SetSelectorBrush(BOX_BRUSH("Starship/Common/color-grading-spinbox-selector", FMargin(1/3.f, 0.0f)))
			.SetSelectorWidth(3.f)
		);
	}

	Instance->Set("ColorGradingPicker.NumericEntry.TextBox", FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("DarkEditableTextBox"));
	Instance->Set("ColorGrading.NormalFont", FAppStyle::GetFontStyle("PropertyWindow.NormalFont"));

	FLinearColor& VectorColor = ColorSettings.StructColors.FindOrAdd(TBaseStructure<FVector>::Get()->GetStructPathName().ToString());
	VectorColor = FColorList::Yellow;
	FLinearColor& RotatorColor = ColorSettings.StructColors.FindOrAdd(TBaseStructure<FRotator>::Get()->GetStructPathName().ToString());
	RotatorColor = FColorList::DarkTurquoise;


	FSlateStyleRegistry::RegisterSlateStyle(*Instance.Get());
}

void FAdvancedWidgetsStyle::Destroy()
{
	if (Instance)
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*Instance.Get());
	}
}

} //namespace

#undef BORDER_BRUSH
