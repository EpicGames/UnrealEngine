// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/ColorGradingSpinBoxStyle.h"
#include "Brushes/SlateNoResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ColorGradingSpinBoxStyle)

FColorGradingSpinBoxStyle::FColorGradingSpinBoxStyle()
	: ActiveBorderBrush(FSlateOptionalBrush())
	, SelectorWidth(1.f)
{
}

void FColorGradingSpinBoxStyle::GetResources(TArray< const FSlateBrush* >& OutBrushes) const
{
	OutBrushes.Add(&BorderBrush);
	OutBrushes.Add(&HoveredBorderBrush);
	OutBrushes.Add(&ActiveBorderBrush);
	OutBrushes.Add(&SelectorBrush);
}

const FName FColorGradingSpinBoxStyle::TypeName(TEXT("FColorGradingSpinBoxStyle"));

const FColorGradingSpinBoxStyle& FColorGradingSpinBoxStyle::GetDefault()
{
	static FColorGradingSpinBoxStyle Default;
	return Default;
}
