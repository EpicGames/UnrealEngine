// Copyright Epic Games, Inc. All Rights Reserved.

#include "Util/ColorGradingUtil.h"
#include "Misc/AssertionMacros.h"

namespace UE::ColorGrading
{

EColorGradingComponent GetColorGradingComponent(EColorGradingColorDisplayMode DisplayMode, int Index)
{
	ensureMsgf(Index >= 0 && Index <= 3, TEXT("Color grading component index %d is outside of expected range (0 to 3)"), Index);

	if (Index == 3)
	{
		return EColorGradingComponent::Luminance;
	}

	switch (DisplayMode)
	{
	case EColorGradingColorDisplayMode::RGB:
		switch (Index)
		{
		case 0:
			return EColorGradingComponent::Red;

		case 1:
			return EColorGradingComponent::Green;

		case 2:
			return EColorGradingComponent::Blue;
		}
		break;

	case EColorGradingColorDisplayMode::HSV:
		switch (Index)
		{
		case 0:
			return EColorGradingComponent::Hue;

		case 1:
			return EColorGradingComponent::Saturation;

		case 2:
			return EColorGradingComponent::Value;
		}
		break;
	}

	return EColorGradingComponent::Red;
}

} //namespace
