// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Util/TrackedVector4PropertyHandle.h"

FTrackedVector4PropertyHandle::FTrackedVector4PropertyHandle()
	: Handle(nullptr)
{
}

FTrackedVector4PropertyHandle::FTrackedVector4PropertyHandle(TWeakPtr<IPropertyHandle> InHandle)
	: Handle(InHandle)
{
}

TSharedPtr<IPropertyHandle> FTrackedVector4PropertyHandle::GetHandle() const
{
	return Handle.Pin();
}

FPropertyAccess::Result FTrackedVector4PropertyHandle::SetValue(const FVector4& InValue, EPropertyValueSetFlags::Type Flags)
{
	FPropertyAccess::Result Result = FPropertyAccess::Fail;

	if (TSharedPtr<IPropertyHandle> PinnedHandle = Handle.Pin())
	{
		bIsSettingValue = true;
		Result = PinnedHandle->SetValue(InValue, Flags);
		bIsSettingValue = false;
	}

	return Result;
}

FPropertyAccess::Result FTrackedVector4PropertyHandle::GetValue(FVector4& OutValue) const
{
	if (TSharedPtr<IPropertyHandle> PinnedHandle = Handle.Pin())
	{
		return PinnedHandle->GetValue(OutValue);
	}

	return FPropertyAccess::Fail;
}

bool FTrackedVector4PropertyHandle::IsSettingValue() const
{
	return bIsSettingValue;
}

bool FTrackedVector4PropertyHandle::IsValidHandle() const
{
	if (TSharedPtr<IPropertyHandle> PinnedHandle = Handle.Pin())
	{
		return PinnedHandle->IsValidHandle();
	}

	return false;
}

#endif