// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/ColorGrading/SColorGradingComponentSpinBox.h"

namespace UE::ColorGrading
{

TArray<FLinearColor> SColorGradingComponentSpinBox::HueGradientColors = {};

SColorGradingComponentSpinBox::~SColorGradingComponentSpinBox()
{
	if (bDragging || PointerDraggingSliderIndex != INDEX_NONE)
	{
		CancelMouseCapture();
	}
}

void SColorGradingComponentSpinBox::Construct(const FArguments& InArgs)
{
	check(InArgs._Style);

	Style = InArgs._Style;
	Component = InArgs._Component;
	ColorGradingMode = InArgs._ColorGradingMode;
	bAllowSpin = InArgs._AllowSpin;

	Interface = InArgs._TypeInterface.IsValid() ? InArgs._TypeInterface : MakeShareable(new TDefaultNumericTypeInterface<float>);

	ValueAttribute = InArgs._Value;
	OnValueChanged = InArgs._OnValueChanged;
	OnBeginSliderMovement = InArgs._OnBeginSliderMovement;
	OnEndSliderMovement = InArgs._OnEndSliderMovement;
	OnQueryCurrentColor = InArgs._OnQueryCurrentColor;

	MinValue = InArgs._MinValue;
	MaxValue = InArgs._MaxValue;
	MinSliderValue = (InArgs._MinSliderValue.Get().IsSet()) ? InArgs._MinSliderValue : MinValue;
	MaxSliderValue = (InArgs._MaxSliderValue.Get().IsSet()) ? InArgs._MaxSliderValue : MaxValue;

	AlwaysUsesDeltaSnap = InArgs._AlwaysUsesDeltaSnap;

	SupportDynamicSliderMaxValue = InArgs._SupportDynamicSliderMaxValue;
	SupportDynamicSliderMinValue = InArgs._SupportDynamicSliderMinValue;
	OnDynamicSliderMaxValueChanged = InArgs._OnDynamicSliderMaxValueChanged;
	OnDynamicSliderMinValueChanged = InArgs._OnDynamicSliderMinValueChanged;

	CachedExternalValue = ValueAttribute.Get();

	InternalValue = (double)CachedExternalValue;

	if (SupportDynamicSliderMaxValue.Get() && CachedExternalValue > GetMaxSliderValue())
	{
		ApplySliderMaxValueChanged(float(CachedExternalValue - GetMaxSliderValue()), true);
	}
	else if (SupportDynamicSliderMinValue.Get() && CachedExternalValue < GetMinSliderValue())
	{
		ApplySliderMinValueChanged(float(CachedExternalValue - GetMinSliderValue()), true);
	}

	UpdateIsSpinRangeUnlimited();

	SliderExponent = InArgs._SliderExponent;

	SliderExponentNeutralValue = InArgs._SliderExponentNeutralValue;

	DistanceDragged = 0.0f;
	PreDragValue = float();

	Delta = InArgs._Delta;
	ShiftMultiplier = InArgs._ShiftMultiplier;
	CtrlMultiplier = InArgs._CtrlMultiplier;
	Sensitivity = InArgs._Sensitivity;
	LinearDeltaSensitivity = InArgs._LinearDeltaSensitivity;

	BorderHoveredBrush = &InArgs._Style->HoveredBorderBrush;
	BorderBrush = &InArgs._Style->BorderBrush;
	BorderActiveBrush = InArgs._Style->ActiveBorderBrush.IsSet() ? &InArgs._Style->ActiveBorderBrush : BorderHoveredBrush;
	SelectorBrush = &InArgs._Style->SelectorBrush;
	SelectorWidth = &InArgs._Style->SelectorWidth;

	bDragging = false;
	PointerDraggingSliderIndex = INDEX_NONE;

	bIsTextChanging = false;
}

int32 SColorGradingComponentSpinBox::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = (bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect);

	const bool bActiveFeedback = bDragging;

	const FSlateBrush* BorderImage = bActiveFeedback ? BorderActiveBrush : (IsHovered() && bAllowSpin) ?
		BorderHoveredBrush :
		BorderBrush;

	// Gradient
	{
		const TArray<FLinearColor> Colors = GetGradientColors();
		int32 NumColors = Colors.Num();

		if (NumColors > 1)
		{
			TArray<FSlateGradientStop> GradientStops;

			for (int32 ColorIndex = 0; ColorIndex < NumColors; ++ColorIndex)
			{
				GradientStops.Add(FSlateGradientStop(AllottedGeometry.GetLocalSize() * (float(ColorIndex) / (NumColors - 1)), Colors[ColorIndex]));
			}

			FSlateDrawElement::MakeGradient(
				OutDrawElements,
				LayerId++,
				AllottedGeometry.ToPaintGeometry(),
				GradientStops,
				EOrientation::Orient_Vertical,
				DrawEffects | ESlateDrawEffect::NoGamma, // Gradient colors are in linear space, so disable gamma to let them blend properly
				FVector4f(BorderBrush->OutlineSettings.CornerRadii)
			);
		}
	}

	// Selector
	if (bAllowSpin)
	{
		const float SelectorLayer = LayerId++;
		const float Value = ValueAttribute.Get();

		float FractionFilled = Fraction((double)Value, (double)GetMinSliderValue(), (double)GetMaxSliderValue());
		const float CachedSliderExponent = SliderExponent.Get();
		if (!FMath::IsNearlyEqual(CachedSliderExponent, 1.f))
		{
			if (SliderExponentNeutralValue.IsSet() && SliderExponentNeutralValue.Get() > GetMinSliderValue() && SliderExponentNeutralValue.Get() < GetMaxSliderValue())
			{
				//Compute a log curve on both side of the neutral value
				float StartFractionFilled = Fraction((double)SliderExponentNeutralValue.Get(), (double)GetMinSliderValue(), (double)GetMaxSliderValue());
				FractionFilled = SpinBoxComputeExponentSliderFraction(FractionFilled, StartFractionFilled, CachedSliderExponent);
			}
			else
			{
				FractionFilled = 1.0f - FMath::Pow(1.0f - FractionFilled, CachedSliderExponent);
			}
		}

		const FVector2f AllottedGeometrySize = AllottedGeometry.GetLocalSize();
		const FVector2f SelectorSize(*SelectorWidth, AllottedGeometrySize.Y - (BorderBrush->OutlineSettings.Width * 2));
		const float SelectorRange = AllottedGeometrySize.X - SelectorSize.X;

		const FVector2f SelectorOffset(
			SelectorRange * FractionFilled,
			1.0f
		);

		// Draw the selector's center
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			SelectorLayer,
			AllottedGeometry.ToPaintGeometry(
				SelectorSize,
				FSlateLayoutTransform(SelectorOffset)
			),
			SelectorBrush,
			DrawEffects,
			SelectorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);
	}

	// Border
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(),
		BorderImage,
		DrawEffects,
		BorderImage->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);

	return LayerId;
}

const bool SColorGradingComponentSpinBox::CommitWithMultiplier(const FPointerEvent& MouseEvent)
{
	return MouseEvent.IsShiftDown() || MouseEvent.IsControlDown();
}

FReply SColorGradingComponentSpinBox::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bAllowSpin && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && PointerDraggingSliderIndex == INDEX_NONE)
	{
		DistanceDragged = 0.f;
		PreDragValue = ValueAttribute.Get();
		InternalValue = (double)PreDragValue;
		PointerDraggingSliderIndex = MouseEvent.GetPointerIndex();
		CachedMousePosition = MouseEvent.GetScreenSpacePosition().IntPoint();

		FReply ReturnReply = FReply::Handled().CaptureMouse(SharedThis(this)).UseHighPrecisionMouseMovement(SharedThis(this)).SetUserFocus(SharedThis(this), EFocusCause::Mouse);
		if (bPreventThrottling)
		{
			ReturnReply.PreventThrottling();
		}
		return ReturnReply;
	}
	
	return FReply::Unhandled();
}

FReply SColorGradingComponentSpinBox::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bAllowSpin && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && PointerDraggingSliderIndex == MouseEvent.GetPointerIndex())
	{
		if (!this->HasMouseCapture())
		{
			// Lost Capture - ensure reset
			bDragging = false;
			PointerDraggingSliderIndex = INDEX_NONE;

			return FReply::Unhandled();
		}

		if (bDragging)
		{
			float CurrentDelta = Delta.Get();
			if (CurrentDelta != float() && !CommitWithMultiplier(MouseEvent))
			{
				InternalValue = FMath::GridSnap(InternalValue, (double)CurrentDelta);
			}

			const float CurrentValue = RoundIfIntegerValue(InternalValue);
			NotifyValueCommitted(CurrentValue);
		}

		bDragging = false;
		PointerDraggingSliderIndex = INDEX_NONE;

		FReply Reply = FReply::Handled().ReleaseMouseCapture();

		if (!MouseEvent.IsTouchEvent())
		{
			Reply.SetMousePos(CachedMousePosition);
		}

		return Reply;
	}

	return FReply::Unhandled();
}

void SColorGradingComponentSpinBox::ApplySliderMaxValueChanged(float SliderDeltaToAdd, bool UpdateOnlyIfHigher)
{
	check(SupportDynamicSliderMaxValue.Get());

	float NewMaxSliderValue = std::numeric_limits<float>::min();

	if (MaxSliderValue.IsSet() && MaxSliderValue.Get().IsSet())
	{
		NewMaxSliderValue = GetMaxSliderValue();

		if ((NewMaxSliderValue + (float)SliderDeltaToAdd > GetMaxSliderValue() && UpdateOnlyIfHigher) || !UpdateOnlyIfHigher)
		{
			NewMaxSliderValue += (float)SliderDeltaToAdd;

			if (!MaxSliderValue.IsBound()) // simple value so we can update it without breaking the mechanic otherwise it must be handle by the callback implementer
			{
				SetMaxSliderValue(NewMaxSliderValue);
			}
		}
	}

	if (OnDynamicSliderMaxValueChanged.IsBound())
	{
		OnDynamicSliderMaxValueChanged.Execute(NewMaxSliderValue, TWeakPtr<SWidget>(AsShared()), true, UpdateOnlyIfHigher);
	}
}

void SColorGradingComponentSpinBox::ApplySliderMinValueChanged(float SliderDeltaToAdd, bool UpdateOnlyIfLower)
{
	check(SupportDynamicSliderMaxValue.Get());

	float NewMinSliderValue = std::numeric_limits<float>::min();

	if (MinSliderValue.IsSet() && MinSliderValue.Get().IsSet())
	{
		NewMinSliderValue = GetMinSliderValue();

		if ((NewMinSliderValue + (float)SliderDeltaToAdd < GetMinSliderValue() && UpdateOnlyIfLower) || !UpdateOnlyIfLower)
		{
			NewMinSliderValue += (float)SliderDeltaToAdd;

			if (!MinSliderValue.IsBound()) // simple value so we can update it without breaking the mechanic otherwise it must be handle by the callback implementer
			{
				SetMinSliderValue(NewMinSliderValue);
			}
		}
	}

	if (OnDynamicSliderMinValueChanged.IsBound())
	{
		OnDynamicSliderMinValueChanged.Execute(NewMinSliderValue, TWeakPtr<SWidget>(AsShared()), true, UpdateOnlyIfLower);
	}
}

/**
* The system calls this method to notify the widget that a mouse moved within it. This event is bubbled.
*
* @param MyGeometry The Geometry of the widget receiving the event
* @param MouseEvent Information about the input event
* @return Whether the event was handled along with possible requests for the system to take action.
*/

FReply SColorGradingComponentSpinBox::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bAllowSpin && PointerDraggingSliderIndex == MouseEvent.GetPointerIndex())
	{
		if (!this->HasMouseCapture())
		{
			// Lost the mouse capture - ensure reset
			bDragging = false;
			PointerDraggingSliderIndex = INDEX_NONE;

			return FReply::Unhandled();
		}

		if (!bDragging)
		{
			DistanceDragged += (float)FMath::Abs(MouseEvent.GetCursorDelta().X);
			if (DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance())
			{
				bDragging = true;
				OnBeginSliderMovement.ExecuteIfBound();
			}

			// Cache the mouse, even if not dragging cache it.
			CachedMousePosition = MouseEvent.GetScreenSpacePosition().IntPoint();
		}
		else
		{
			double NewValue = 0.0;

			// Increments the spin based on delta mouse movement.

			// A minimum slider width to use for calculating deltas in the slider-range space
			const float MinSliderWidth = 100.f;
			float SliderWidthInSlateUnits = FMath::Max((float)MyGeometry.GetDrawSize().X, MinSliderWidth);

			if (MouseEvent.IsAltDown())
			{
				float DeltaToAdd = (float)MouseEvent.GetCursorDelta().X / SliderWidthInSlateUnits;

				if (SupportDynamicSliderMaxValue.Get() && (float)InternalValue == GetMaxSliderValue())
				{
					ApplySliderMaxValueChanged(DeltaToAdd, false);
				}
				else if (SupportDynamicSliderMinValue.Get() && (float)InternalValue == GetMinSliderValue())
				{
					ApplySliderMinValueChanged(DeltaToAdd, false);
				}
			}

			ECommitMethod CommitMethod = CommittedViaSpin;

			double Step = 0.1;

			if (MouseEvent.IsControlDown())
			{
				Step *= CtrlMultiplier.Get();
				CommitMethod = CommittedViaSpinMultiplier;
			}
			else if (MouseEvent.IsShiftDown())
			{
				Step *= ShiftMultiplier.Get();
				CommitMethod = CommittedViaSpinMultiplier;
			}

			const float MouseXMovement = MouseEvent.GetCursorDelta().X * Sensitivity.Get(1.f);

			//if we have a range to draw in
			if (!bUnlimitedSpinRange)
			{
				bool HasValidExponentNeutralValue = SliderExponentNeutralValue.IsSet() && SliderExponentNeutralValue.Get() > GetMinSliderValue() && SliderExponentNeutralValue.Get() < GetMaxSliderValue();

				const float CachedSliderExponent = SliderExponent.Get();
				// The amount currently filled in the spinbox, needs to be calculated to do deltas correctly.
				float FractionFilled = Fraction(InternalValue, (double)GetMinSliderValue(), (double)GetMaxSliderValue());

				if (!FMath::IsNearlyEqual(CachedSliderExponent, 1.0f))
				{
					if (HasValidExponentNeutralValue)
					{
						//Compute a log curve on both side of the neutral value
						float StartFractionFilled = Fraction((double)SliderExponentNeutralValue.Get(), (double)GetMinSliderValue(), (double)GetMaxSliderValue());
						FractionFilled = SpinBoxComputeExponentSliderFraction(FractionFilled, StartFractionFilled, CachedSliderExponent);
					}
					else
					{
						FractionFilled = 1.0f - FMath::Pow(1.0f - FractionFilled, CachedSliderExponent);
					}
				}
				FractionFilled *= SliderWidthInSlateUnits;

				// Now add the delta to the fraction filled, this causes the spin.
				FractionFilled += (float)(MouseXMovement * Step);

				// Clamp the fraction to be within the bounds of the geometry.
				FractionFilled = FMath::Clamp(FractionFilled, 0.0f, SliderWidthInSlateUnits);

				// Convert the fraction filled to a percent.
				float Percent = FMath::Clamp(FractionFilled / SliderWidthInSlateUnits, 0.0f, 1.0f);
				if (!FMath::IsNearlyEqual(CachedSliderExponent, 1.0f))
				{
					// Have to convert the percent to the proper value due to the exponent component to the spin.
					if (HasValidExponentNeutralValue)
					{
						//Compute a log curve on both side of the neutral value
						float StartFractionFilled = Fraction(SliderExponentNeutralValue.Get(), GetMinSliderValue(), GetMaxSliderValue());
						Percent = SpinBoxComputeExponentSliderFraction(Percent, StartFractionFilled, 1.0f / CachedSliderExponent);
					}
					else
					{
						Percent = 1.0f - FMath::Pow(1.0f - Percent, 1.0f / CachedSliderExponent);
					}


				}

				NewValue = FMath::LerpStable<double>((double)GetMinSliderValue(), (double)GetMaxSliderValue(), Percent);
			}
			else
			{
				// If this control has a specified delta and sensitivity then we use that instead of the current value for determining how much to change.
				const double Sign = MouseXMovement ? 1.0 : -1.0;

				if (LinearDeltaSensitivity.IsSet() && LinearDeltaSensitivity.Get() != 0 && Delta.IsSet() && Delta.Get() > 0)
				{
					const double MouseDelta = FMath::Abs(MouseXMovement / (float)LinearDeltaSensitivity.Get());
					NewValue = InternalValue + (Sign * MouseDelta * FMath::Pow((double)Delta.Get(), (double)SliderExponent.Get())) * Step;
				}
				else
				{
					const double MouseDelta = FMath::Abs(MouseXMovement / SliderWidthInSlateUnits);
					const double CurrentValue = FMath::Clamp<double>(FMath::Abs(InternalValue), 1.0, (double)std::numeric_limits<float>::max());
					NewValue = InternalValue + (Sign * MouseDelta * FMath::Pow((double)CurrentValue, (double)SliderExponent.Get())) * Step;
				}
			}

			float RoundedNewValue = RoundIfIntegerValue(NewValue);
			CommitValue(RoundedNewValue, NewValue, CommitMethod, ETextCommit::OnEnter);
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FCursorReply SColorGradingComponentSpinBox::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if (!bAllowSpin)
	{
		return FCursorReply::Cursor(EMouseCursor::Default);
	}

	return bDragging ?
		FCursorReply::Cursor(EMouseCursor::None) :
		FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
}

void SColorGradingComponentSpinBox::CommitValue(float NewValue, double NewSpinValue, ECommitMethod CommitMethod, ETextCommit::Type OriginalCommitInfo)
{
	if (!bAllowSpin)
	{
		return;
	}

	if (CommitMethod == CommittedViaSpin)
	{
		const float LocalMinSliderValue = GetMinSliderValue();
		const float LocalMaxSliderValue = GetMaxSliderValue();
		NewValue = FMath::Clamp<float>(NewValue, LocalMinSliderValue, LocalMaxSliderValue);
		NewSpinValue = FMath::Clamp<double>(NewSpinValue, (double)LocalMinSliderValue, (double)LocalMaxSliderValue);
	}

	{
		const float LocalMinValue = GetMinValue();
		const float LocalMaxValue = GetMaxValue();
		NewValue = FMath::Clamp<float>(NewValue, LocalMinValue, LocalMaxValue);
		NewSpinValue = FMath::Clamp<double>(NewSpinValue, (double)LocalMinValue, (double)LocalMaxValue);
	}

	if (!ValueAttribute.IsBound())
	{
		ValueAttribute.Set(NewValue);
	}

	// If not in spin mode, there is no need to jump to the value from the external source, continue to use the committed value.
	if (CommitMethod == CommittedViaSpin)
	{
		const float CurrentValue = ValueAttribute.Get();
		// This will detect if an external force has changed the value. Internally it will abandon the delta calculated this tick and update the internal value instead.
		if (CurrentValue != CachedExternalValue)
		{
			NewValue = CurrentValue;
			NewSpinValue = (double)CurrentValue;
		}
	}

	// Update the internal value, this needs to be done before rounding.
	InternalValue = NewSpinValue;

	const bool bAlwaysUsesDeltaSnap = GetAlwaysUsesDeltaSnap();
	// If needed, round this value to the delta. Internally the value is not held to the Delta but externally it appears to be.
	if (CommitMethod == CommittedViaSpin || bAlwaysUsesDeltaSnap)
	{
		float CurrentDelta = Delta.Get();
		if (CurrentDelta != float())
		{
			NewValue = FMath::GridSnap<float>(NewValue, CurrentDelta); // snap numeric point value to nearest Delta
		}
	}

	// Update the max slider value based on the current value if we're in dynamic mode
	if (SupportDynamicSliderMaxValue.Get() && ValueAttribute.Get() > GetMaxSliderValue())
	{
		ApplySliderMaxValueChanged(float(ValueAttribute.Get() - GetMaxSliderValue()), true);
	}
	else if (SupportDynamicSliderMinValue.Get() && ValueAttribute.Get() < GetMinSliderValue())
	{
		ApplySliderMinValueChanged(float(ValueAttribute.Get() - GetMinSliderValue()), true);
	}

	OnValueChanged.ExecuteIfBound(NewValue);

	if (!ValueAttribute.IsBound())
	{
		ValueAttribute.Set(NewValue);
	}

	// Update the cache of the external value to what the user believes the value is now.
	const float CurrentValue = ValueAttribute.Get();
	if (CachedExternalValue != CurrentValue)
	{
		CachedExternalValue = ValueAttribute.Get();
	}

	// This ensures that dragging is cleared if focus has been removed from this widget in one of the delegate calls, such as when spawning a modal dialog.
	if (!this->HasMouseCapture())
	{
		bDragging = false;
		PointerDraggingSliderIndex = INDEX_NONE;
	}
}

void SColorGradingComponentSpinBox::NotifyValueCommitted(float CurrentValue) const
{
	// The internal value will have been clamped and rounded to the delta at this point, but integer values may still need to be rounded
	// if the delta is 0.
	OnEndSliderMovement.ExecuteIfBound(CurrentValue);
}

float SColorGradingComponentSpinBox::Fraction(double InValue, double InMinValue, double InMaxValue)
{
	const double HalfMax = InMaxValue * 0.5;
	const double HalfMin = InMinValue * 0.5;
	const double HalfVal = InValue * 0.5;

	return (float)FMath::Clamp((HalfVal - HalfMin) / (HalfMax - HalfMin), 0.0, 1.0);
}

const TArray<FLinearColor>& SColorGradingComponentSpinBox::GetHueGradientColors()
{
	if (HueGradientColors.IsEmpty())
	{
		for (int32 i = 0; i < 7; ++i)
		{
			HueGradientColors.Add(FLinearColor((i % 6) * 60.f, 1.f, 1.f).HSVToLinearRGB());
		}
	}

	return HueGradientColors;
}

float SColorGradingComponentSpinBox::RoundIfIntegerValue(double ValueToRound) const
{
	constexpr bool bIsIntegral = TIsIntegral<float>::Value;
	constexpr bool bCanBeRepresentedInDouble = std::numeric_limits<double>::digits >= std::numeric_limits<float>::digits;
	if (bIsIntegral && !bCanBeRepresentedInDouble)
	{
		return (float)FMath::Clamp<double>(FMath::FloorToDouble(ValueToRound + 0.5), -1.0 * (double)(1ll << std::numeric_limits<double>::digits), (double)(1ll << std::numeric_limits<double>::digits));
	}
	else if (bIsIntegral)
	{
		return (float)FMath::Clamp<double>(FMath::FloorToDouble(ValueToRound + 0.5), (double)std::numeric_limits<float>::lowest(), (double)std::numeric_limits<float>::max());
	}
	else
	{
		return (float)FMath::Clamp<double>(ValueToRound, (double)std::numeric_limits<float>::lowest(), (double)std::numeric_limits<float>::max());
	}
}

void SColorGradingComponentSpinBox::CancelMouseCapture()
{
	bDragging = false;
	PointerDraggingSliderIndex = INDEX_NONE;

	InternalValue = (double)PreDragValue;
	NotifyValueCommitted(PreDragValue);
}

TArray<FLinearColor> SColorGradingComponentSpinBox::GetGradientColors() const
{
	const EColorGradingComponent DisplayedComponent = Component.Get();
	const bool bIsOffset = ColorGradingMode.Get() == EColorGradingModes::Offset;

	// Create gradients
	switch (DisplayedComponent)
	{
	case EColorGradingComponent::Red:
		if (bIsOffset)
		{
			return
			{
				FLinearColor(0, 1, 1, 1),
				FLinearColor::Black,
				FLinearColor::Red
			};
		}

		return
		{
			FLinearColor::Black,
			FLinearColor::Red
		};

	case EColorGradingComponent::Green:
		if (bIsOffset)
		{
			return
			{
				FLinearColor(1, 0, 1, 1),
				FLinearColor::Black,
				FLinearColor::Green
			};
		}

		return
		{
			FLinearColor::Black,
			FLinearColor::Green
		};

	case EColorGradingComponent::Blue:
		if (bIsOffset)
		{
			return
			{
				FLinearColor(1, 1, 0, 1),
				FLinearColor::Black,
				FLinearColor::Blue
			};
		}

		return
		{
			FLinearColor::Black,
			FLinearColor::Blue
		};

	case EColorGradingComponent::Luminance:
		{
			const FLinearColor RGBColor = bIsOffset ? FLinearColor::White : GetCurrentRGBColor();
			return
			{
				FLinearColor::Black,
				RGBColor
			};
		}

	case EColorGradingComponent::Hue:
		return GetHueGradientColors();

	case EColorGradingComponent::Saturation:
		{
			const FLinearColor HSVColor = GetCurrentHSVColor();
			return
			{
				FLinearColor(HSVColor.R, 0.f, HSVColor.B).HSVToLinearRGB(),
				FLinearColor(HSVColor.R, 0.5f, HSVColor.B).HSVToLinearRGB(),
				FLinearColor(HSVColor.R, 1.f, HSVColor.B).HSVToLinearRGB()
			};
		}

	case EColorGradingComponent::Value:
		{
			const FLinearColor HSVColor = GetCurrentHSVColor();
			return
			{
				FLinearColor(HSVColor.R, HSVColor.G, 0.f).HSVToLinearRGB(),
				FLinearColor(HSVColor.R, HSVColor.G, 1.f).HSVToLinearRGB(),
				FLinearColor(HSVColor.R, HSVColor.G, 2.f).HSVToLinearRGB()
			};
		}
	}

	return TArray<FLinearColor>();
}

FLinearColor SColorGradingComponentSpinBox::GetCurrentRGBColor() const
{
	if (OnQueryCurrentColor.IsBound())
	{
		FVector4 ColorComponents;
		OnQueryCurrentColor.Execute(ColorComponents);

		// This component is luminance, but our RGB representation will use it as alpha, so set it to 100%
		ColorComponents.W = 1;

		return FLinearColor(ColorComponents);
	}

	return FLinearColor::White;
}

FLinearColor SColorGradingComponentSpinBox::GetCurrentHSVColor() const
{
	return GetCurrentRGBColor().LinearRGBToHSV();
}

} //namespace