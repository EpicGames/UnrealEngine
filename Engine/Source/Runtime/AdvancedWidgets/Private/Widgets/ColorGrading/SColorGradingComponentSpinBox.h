// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/ColorGrading/ColorGradingCommon.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "InputCoreTypes.h"
#include "HAL/Platform.h"
#include "Misc/Attribute.h"
#include "Styling/AdvancedWidgetsStyle.h"
#include "Styling/ColorGradingSpinBoxStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ColorGrading
{

/**
 * A modified SpinBox that shows a selector along a gradient to indicate a color component for color grading.
 * Allows spinning via mouse drag if enabled, but does not allow text entry like a standard SpinBox.
 */
class SColorGradingComponentSpinBox
	: public SCompoundWidget
{
public:

	/** Notification for numeric value change */
	DECLARE_DELEGATE_OneParam(FOnValueChanged, float);

	/** Notification when the max/min spinner values are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	DECLARE_DELEGATE_FourParams(FOnDynamicSliderMinMaxValueChanged, float, TWeakPtr<SWidget>, bool, bool);

	SLATE_BEGIN_ARGS(SColorGradingComponentSpinBox)
		: _Style(&UE::AdvancedWidgets::FAdvancedWidgetsStyle::Get().GetWidgetStyle<FColorGradingSpinBoxStyle>("ColorGradingSpinBox"))
		, _Value(0)
		, _Component(EColorGradingComponent::Red)
		, _ColorGradingMode(EColorGradingModes::Invalid)
		, _MinValue(0)
		, _MaxValue(2)
		, _Delta(0)
		, _ShiftMultiplier(10.f)
		, _CtrlMultiplier(0.1f)
		, _Sensitivity(1.0f)
		, _SupportDynamicSliderMaxValue(false)
		, _SupportDynamicSliderMinValue(false)
		, _SliderExponent(1.f)
		, _OnValueChanged()
		, _OnQueryCurrentColor()
		, _AllowSpin(true)
		{}

		/** The style used to draw this spinbox */
		SLATE_STYLE_ARGUMENT(FColorGradingSpinBoxStyle, Style)

		/** The value to display */
		SLATE_ATTRIBUTE(float, Value)
		/** The component being displayed */
		SLATE_ATTRIBUTE(EColorGradingComponent, Component)
		/** The mode of the associated color grading wheel */
		SLATE_ATTRIBUTE(EColorGradingModes, ColorGradingMode)
		/** The minimum value that can be entered into the text edit box */
		SLATE_ATTRIBUTE(TOptional<float>, MinValue)
		/** The maximum value that can be entered into the text edit box */
		SLATE_ATTRIBUTE(TOptional<float>, MaxValue)
		/** The minimum value that can be specified by using the slider, defaults to MinValue */
		SLATE_ATTRIBUTE(TOptional<float>, MinSliderValue)
		/** The maximum value that can be specified by using the slider, defaults to MaxValue */
		SLATE_ATTRIBUTE(TOptional<float>, MaxSliderValue)
		/** Whether typed values should use delta snapping, defaults to false */
		SLATE_ATTRIBUTE(bool, AlwaysUsesDeltaSnap)
		/** Delta to increment the value as the slider moves.  If not specified will determine automatically */
		SLATE_ATTRIBUTE(float, Delta)
		/** Multiplier to use when shift is held down */
		SLATE_ATTRIBUTE(float, ShiftMultiplier)
		/** Multiplier to use when ctrl is held down */
		SLATE_ATTRIBUTE(float, CtrlMultiplier)
		/** Multiplier to apply to all mouse movement */
		SLATE_ATTRIBUTE(float, Sensitivity)
		/** If we're an unbounded spinbox, what value do we divide mouse movement by before multiplying by Delta. Requires Delta to be set. */
		SLATE_ATTRIBUTE(int32, LinearDeltaSensitivity)
		/** Tell us if we want to support dynamically changing of the max value using alt */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMaxValue)
		/** Tell us if we want to support dynamically changing of the min value using alt */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMinValue)
		/** Called right after the max slider value is changed (only relevant if SupportDynamicSliderMaxValue is true) */
		SLATE_EVENT(FOnDynamicSliderMinMaxValueChanged, OnDynamicSliderMaxValueChanged)
		/** Called right after the min slider value is changed (only relevant if SupportDynamicSliderMinValue is true) */
		SLATE_EVENT(FOnDynamicSliderMinMaxValueChanged, OnDynamicSliderMinValueChanged)
		/** Use exponential scale for the slider */
		SLATE_ATTRIBUTE(float, SliderExponent)
		/** When use exponential scale for the slider which is the neutral value */
		SLATE_ATTRIBUTE(float, SliderExponentNeutralValue)
		/** Step to increment or decrement the value by when scrolling the mouse wheel. If not specified will determine automatically */
		SLATE_ATTRIBUTE(TOptional<float>, WheelStep)
		/** Called when the value is changed by slider or typing */
		SLATE_EVENT(FOnValueChanged, OnValueChanged)
		/** Called right before the slider begins to move */
		SLATE_EVENT(FSimpleDelegate, OnBeginSliderMovement)
		/** Called right after the slider handle is released by the user */
		SLATE_EVENT(FOnValueChanged, OnEndSliderMovement)
		/** Callback to get the current FVector4 color value (used to update the background gradient) */
		SLATE_EVENT(FOnGetCurrentVector4Value, OnQueryCurrentColor)
		/** Provide custom type conversion functionality to this spin box */
		SLATE_ARGUMENT(TSharedPtr<INumericTypeInterface<float>>, TypeInterface)
		/** Whether or not the user should be able to change the value by dragging with the mouse cursor */
		SLATE_ARGUMENT(bool, AllowSpin)

	SLATE_END_ARGS()

	SColorGradingComponentSpinBox() = default;

	virtual ~SColorGradingComponentSpinBox();

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs);

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	const bool CommitWithMultiplier(const FPointerEvent& MouseEvent);

	/**
	 * The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/**
	 * The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	void ApplySliderMaxValueChanged(float SliderDeltaToAdd, bool UpdateOnlyIfHigher);

	void ApplySliderMinValueChanged(float SliderDeltaToAdd, bool UpdateOnlyIfLower);

	/**
	 * The system calls this method to notify the widget that a mouse moved within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	/** Return the Value attribute */
	TAttribute<float> GetValueAttribute() const { return ValueAttribute; }

	/** See the Value attribute */
	float GetValue() const { return ValueAttribute.Get(); }
	void SetValue(const TAttribute<float>& InValueAttribute)
	{
		ValueAttribute = InValueAttribute;
		const float LocalValueAttribute = ValueAttribute.Get();
		CommitValue(LocalValueAttribute, (double)LocalValueAttribute, ECommitMethod::CommittedViaCode, ETextCommit::Default);
	}

	/** See the MinValue attribute */
	float GetMinValue() const { return MinValue.Get().Get(std::numeric_limits<float>::lowest()); }
	void SetMinValue(const TAttribute<TOptional<float>>& InMinValue)
	{
		MinValue = InMinValue;
		UpdateIsSpinRangeUnlimited();
	}

	/** See the MaxValue attribute */
	float GetMaxValue() const { return MaxValue.Get().Get(std::numeric_limits<float>::max()); }
	void SetMaxValue(const TAttribute<TOptional<float>>& InMaxValue)
	{
		MaxValue = InMaxValue;
		UpdateIsSpinRangeUnlimited();
	}

	/** See the MinSliderValue attribute */
	bool IsMinSliderValueBound() const { return MinSliderValue.IsBound(); }

	float GetMinSliderValue() const { return MinSliderValue.Get().Get(std::numeric_limits<float>::lowest()); }
	void SetMinSliderValue(const TAttribute<TOptional<float>>& InMinSliderValue)
	{
		MinSliderValue = (InMinSliderValue.Get().IsSet()) ? InMinSliderValue : MinValue;
		UpdateIsSpinRangeUnlimited();
	}

	/** See the MaxSliderValue attribute */
	bool IsMaxSliderValueBound() const { return MaxSliderValue.IsBound(); }

	float GetMaxSliderValue() const { return MaxSliderValue.Get().Get(std::numeric_limits<float>::max()); }
	void SetMaxSliderValue(const TAttribute<TOptional<float>>& InMaxSliderValue)
	{
		MaxSliderValue = (InMaxSliderValue.Get().IsSet()) ? InMaxSliderValue : MaxValue;;
		UpdateIsSpinRangeUnlimited();
	}

	/** See the AlwaysUsesDeltaSnap attribute */
	bool GetAlwaysUsesDeltaSnap() const { return AlwaysUsesDeltaSnap.Get(); }
	void SetAlwaysUsesDeltaSnap(bool bNewValue) { AlwaysUsesDeltaSnap.Set(bNewValue); }

	/** See the Delta attribute */
	float GetDelta() const { return Delta.Get(); }
	void SetDelta(float InDelta) { Delta.Set(InDelta); }

	/** See the SliderExponent attribute */
	float GetSliderExponent() const { return SliderExponent.Get(); }
	void SetSliderExponent(const TAttribute<float>& InSliderExponent) { SliderExponent = InSliderExponent; }

	const FColorGradingSpinBoxStyle* GetWidgetStyle() const { return Style; }
	void SetWidgetStyle(const FColorGradingSpinBoxStyle* InStyle) { Style = InStyle; }
	void InvalidateStyle() { Invalidate(EInvalidateWidgetReason::Layout); }

protected:
	/** How user changed the value in the spinbox */
	enum ECommitMethod
	{
		CommittedViaSpin,
		CommittedViaCode,
		CommittedViaSpinMultiplier
	};

	/**
	 * Call this method when the user's interaction has changed the value
	 *
	 * @param NewValue               Value resulting from the user's interaction
	 * @param CommitMethod           Did the user type in the value or spin to it.
	 * @param OriginalCommitInfo	 If the user typed in the value, information about the source of the commit
	 */
	void CommitValue(float NewValue, double NewSpinValue, ECommitMethod CommitMethod, ETextCommit::Type OriginalCommitInfo);

	void NotifyValueCommitted(float CurrentValue) const;

	/** Calculates range fraction. Possible to use on full numeric range  */
	static float Fraction(double InValue, double InMinValue, double InMaxValue);

private:

	/** Get the gradient stops for a hue slider. These are lazily generated and cached for future calls. */
	static const TArray<FLinearColor>& GetHueGradientColors();
	static TArray<FLinearColor> HueGradientColors;

	TAttribute<float> ValueAttribute;
	FOnValueChanged OnValueChanged;
	FSimpleDelegate OnBeginSliderMovement;
	FOnValueChanged OnEndSliderMovement;
	FOnGetCurrentVector4Value OnQueryCurrentColor;

	/** Interface that defines conversion functionality for the templated type */
	TSharedPtr<INumericTypeInterface<float>> Interface;

	/** True when no range is specified, spinner can be spun indefinitely */
	bool bUnlimitedSpinRange;
	void UpdateIsSpinRangeUnlimited()
	{
		bUnlimitedSpinRange = !((MinValue.Get().IsSet() && MaxValue.Get().IsSet()) || (MinSliderValue.Get().IsSet() && MaxSliderValue.Get().IsSet()));
	}

	const FColorGradingSpinBoxStyle* Style;

	const FSlateBrush* BorderHoveredBrush;
	const FSlateBrush* BorderActiveBrush;
	const FSlateBrush* BorderBrush;
	const FSlateBrush* SelectorBrush;
	const float* SelectorWidth;

	bool bAllowSpin;
	float DistanceDragged;
	TAttribute<float> Delta;
	TAttribute<float> ShiftMultiplier;
	TAttribute<float> CtrlMultiplier;
	TAttribute<float> Sensitivity;
	TAttribute<int32> LinearDeltaSensitivity;
	TAttribute<float> SliderExponent;
	TAttribute<float> SliderExponentNeutralValue;
	TAttribute<TOptional<float>> MinValue;
	TAttribute<TOptional<float>> MaxValue;
	TAttribute<TOptional<float>> MinSliderValue;
	TAttribute<TOptional<float>> MaxSliderValue;
	TAttribute<bool> AlwaysUsesDeltaSnap;
	TAttribute<bool> SupportDynamicSliderMaxValue;
	TAttribute<bool> SupportDynamicSliderMinValue;
	FOnDynamicSliderMinMaxValueChanged OnDynamicSliderMaxValueChanged;
	FOnDynamicSliderMinMaxValueChanged OnDynamicSliderMinValueChanged;
	TAttribute<EColorGradingComponent> Component;
	TAttribute<EColorGradingModes> ColorGradingMode;

	/**
	 * Rounds the submitted value to the correct value if it's an integer.
	 * For int64, not all values can be represented by a double. We can only round until we reach that limit.
	 * This function should only be used when we drag the value. We accept that we can't drag huge numbers.
	 */
	float RoundIfIntegerValue(double ValueToRound) const;

	void CancelMouseCapture();

	/** Generate the gradient stops to display in the background based on the current color and viewed component. */
	TArray<FLinearColor> GetGradientColors() const;

	/** Get the currently selected color in linear RGB space. */
	FLinearColor GetCurrentRGBColor() const;

	/** Get the currently selected color in HSV space. */
	FLinearColor GetCurrentHSVColor() const;

	/** Tracks which cursor is currently dragging the slider (e.g., the mouse cursor or a specific finger) */
	int32 PointerDraggingSliderIndex;

	/** Cached mouse position to restore after scrolling. */
	FIntPoint CachedMousePosition;

	/**
	 * This value represents what the spinbox believes the value to be, regardless of delta and the user binding to an int.
	 * The spinbox will always count using floats between values, this is important to keep it flowing smoothly and feeling right,
	 * and most importantly not conflicting with the user truncating the value to an int.
	 */
	double InternalValue;

	/** The state of InternalValue before a drag operation was started */
	float PreDragValue;

	/**
	 * This is the cached value the user believes it to be (usually different due to truncation to an int). Used for identifying
	 * external forces on the spinbox and syncing the internal value to them. Synced when a value is committed to the spinbox.
	 */
	float CachedExternalValue;

	/** Whether the user is dragging the slider */
	bool bDragging;

	/** Re-entrant guard for the text changed handler */
	bool bIsTextChanging;

	/*
	 * Holds whether or not to prevent throttling during mouse capture
	 * When true, the viewport will be updated with every single change to the value during dragging
	 */
	bool bPreventThrottling;

	/** Does this spin box have the mouse wheel feature enabled? */
	bool bEnableWheel = true;

	/** True to broadcast every time we type. */
	bool bBroadcastValueChangesPerKey = false;
};

} //namespace
