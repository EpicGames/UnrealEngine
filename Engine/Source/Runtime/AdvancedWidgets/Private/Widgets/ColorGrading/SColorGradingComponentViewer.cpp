// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/ColorGrading/SColorGradingComponentViewer.h"

#include "SColorGradingComponentSpinBox.h"
#include "Math/NumericLimits.h"
#include "Styling/AdvancedWidgetsStyle.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "ColorGradingEditor"

namespace UE::ColorGrading
{

SColorGradingComponentViewer::~SColorGradingComponentViewer()
{
}

void SColorGradingComponentViewer::Construct(const FArguments& InArgs)
{
	OptionalValue = InArgs._Value;
	Component = InArgs._Component;
	MinValue = InArgs._MinValue;
	MaxValue = InArgs._MaxValue;
	MinSliderValue = InArgs._MinSliderValue;
	MaxSliderValue = InArgs._MaxSliderValue;

	const bool bUseCompactDisplay = InArgs._UseCompactDisplay;
	const auto& AdvancedWidgetsStyle = UE::AdvancedWidgets::FAdvancedWidgetsStyle::Get();

	TSharedPtr<SHorizontalBox> HorizontalBox;

	this->ChildSlot
	[
		SNew(SBox)
		.HeightOverride(20)
		.HAlign(HAlign_Fill)
		.ToolTipText(this, &SColorGradingComponentViewer::GetComponentToolTipText)
		[
			SAssignNew(HorizontalBox, SHorizontalBox)
		]
	];

	HorizontalBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(8)
			[
				SNew(STextBlock)
				.Text(this, &SColorGradingComponentViewer::GetComponentLabelText)
			]
		];

	if (!bUseCompactDisplay)
	{
		HorizontalBox->AddSlot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.Padding(8.f, 0.f, 0.f, 0.f)
			[
				SAssignNew(GradientSpinBox, SColorGradingComponentSpinBox)
				.Value(this, &SColorGradingComponentViewer::GetValue)
				.Component(InArgs._Component)
				.ColorGradingMode(InArgs._ColorGradingMode)
				.OnValueChanged(InArgs._OnValueChanged)
				.OnBeginSliderMovement(InArgs._OnBeginSliderMovement)
				.OnEndSliderMovement(InArgs._OnEndSliderMovement)
				.ShiftMultiplier(InArgs._ShiftMultiplier)
				.CtrlMultiplier(InArgs._CtrlMultiplier)
				.SupportDynamicSliderMinValue(InArgs._SupportDynamicSliderMinValue)
				.SupportDynamicSliderMaxValue(InArgs._SupportDynamicSliderMaxValue)
				.OnDynamicSliderMinValueChanged(InArgs._OnDynamicSliderMinValueChanged)
				.OnDynamicSliderMaxValueChanged(InArgs._OnDynamicSliderMaxValueChanged)
				.OnQueryCurrentColor(InArgs._OnQueryCurrentColor)
				.MinValue(InArgs._MinValue)
				.MaxValue(InArgs._MaxValue)
				.MinSliderValue(InArgs._MinSliderValue)
				.MaxSliderValue(InArgs._MaxSliderValue)
				.SliderExponent(InArgs._SliderExponent)
				.SliderExponentNeutralValue(InArgs._SliderExponentNeutralValue)
				.Delta(InArgs._Delta)
				.TypeInterface(InArgs._TypeInterface)
				.IsEnabled(this, &SWidget::IsEnabled)
				.AllowSpin(InArgs._AllowSpin)
			];
	}

	TSharedPtr<SBox> NumericEntryContainer;
	SAssignNew(NumericEntryContainer, SBox)
	[
		SAssignNew(NumericEntryBox, SNumericEntryBox<float>)
		.SpinBoxStyle(&AdvancedWidgetsStyle.GetWidgetStyle<FSpinBoxStyle>("ColorGradingComponentViewer.NumericEntry"))
		.Font(AdvancedWidgetsStyle.GetFontStyle("ColorGrading.NormalFont"))
		.EditableTextBoxStyle(&AdvancedWidgetsStyle.GetWidgetStyle<FEditableTextBoxStyle>("ColorGradingComponentViewer.NumericEntry.TextBox"))
		.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
		.Value(InArgs._Value)
		.OnValueChanged(InArgs._OnValueChanged)
		.OnBeginSliderMovement(InArgs._OnBeginSliderMovement)
		.OnEndSliderMovement(InArgs._OnEndSliderMovement)
		.AllowSpin(true)
		.ShiftMultiplier(InArgs._ShiftMultiplier)
		.CtrlMultiplier(InArgs._CtrlMultiplier)
		.SupportDynamicSliderMinValue(InArgs._SupportDynamicSliderMinValue)
		.SupportDynamicSliderMaxValue(InArgs._SupportDynamicSliderMaxValue)
		.OnDynamicSliderMinValueChanged(InArgs._OnDynamicSliderMinValueChanged)
		.OnDynamicSliderMaxValueChanged(InArgs._OnDynamicSliderMaxValueChanged)
		.MinValue(InArgs._MinValue)
		.MaxValue(InArgs._MaxValue)
		.MinSliderValue(InArgs._MinSliderValue)
		.MaxSliderValue(InArgs._MaxSliderValue)
		.MinFractionalDigits(InArgs._MinFractionalDigits)
		.MaxFractionalDigits(InArgs._MaxFractionalDigits)
		.SliderExponent(InArgs._SliderExponent)
		.SliderExponentNeutralValue(InArgs._SliderExponentNeutralValue)
		.Delta(InArgs._Delta)
		.TypeInterface(InArgs._TypeInterface)
		.IsEnabled(this, &SWidget::IsEnabled)
		.AllowSpin(InArgs._AllowSpin)
	];


	const FMargin NumericEntryPadding(bUseCompactDisplay ? 4.f : 8.f, 0.f, 0.f, 0.f);

	if (bUseCompactDisplay)
	{
		HorizontalBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillWidth(1.0f)
			.Padding(NumericEntryPadding)
			[
				NumericEntryContainer.ToSharedRef()
			];
	}
	else
	{
		HorizontalBox->AddSlot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.AutoWidth()
			.Padding(NumericEntryPadding)
			[
				NumericEntryContainer.ToSharedRef()
			];

		NumericEntryContainer->SetWidthOverride(56);
	}
}

float SColorGradingComponentViewer::GetMaxSliderValue() const
{
	if (MaxSliderValue.IsSet() && MaxSliderValue.Get().IsSet())
	{
		return MaxSliderValue.Get().GetValue();
	}

	return MaxValue.Get().Get(TNumericLimits<float>::Max());
}

float SColorGradingComponentViewer::GetMinSliderValue() const
{
	if (MinSliderValue.IsSet() && MinSliderValue.Get().IsSet())
	{
		return MinSliderValue.Get().GetValue();
	}

	return MinValue.Get().Get(TNumericLimits<float>::Min());
}

float SColorGradingComponentViewer::GetValue() const
{
	if (OptionalValue.IsSet())
	{
		if (const float* Value = OptionalValue.Get().GetPtrOrNull())
		{
			return *Value;
		}
	}

	return 0.0f;
}

FText SColorGradingComponentViewer::GetComponentLabelText() const
{
	switch (Component.Get())
	{
	case EColorGradingComponent::Red:
		return LOCTEXT("ColorWheel_RedComponentLabel", "R");

	case EColorGradingComponent::Green:
		return LOCTEXT("ColorWheel_GreenComponentLabel", "G");

	case EColorGradingComponent::Blue:
		return LOCTEXT("ColorWheel_BlueComponentLabel", "B");

	case EColorGradingComponent::Hue:
		return LOCTEXT("ColorWheel_HueComponentLabel", "H");

	case EColorGradingComponent::Saturation:
		return LOCTEXT("ColorWheel_SaturationComponentLabel", "S");

	case EColorGradingComponent::Value:
		return LOCTEXT("ColorWheel_ValueComponentLabel", "V");

	case EColorGradingComponent::Luminance:
		return LOCTEXT("ColorWheel_LuminanceComponentLabel", "Y");
	}

	return FText::GetEmpty();
}

FText SColorGradingComponentViewer::GetComponentToolTipText() const
{
	switch (Component.Get())
	{
	case EColorGradingComponent::Red:
		return LOCTEXT("ColorWheel_RedComponentToolTip", "Red");

	case EColorGradingComponent::Green:
		return LOCTEXT("ColorWheel_GreenComponentToolTip", "Green");

	case EColorGradingComponent::Blue:
		return LOCTEXT("ColorWheel_BlueComponentToolTip", "Blue");

	case EColorGradingComponent::Hue:
		return LOCTEXT("ColorWheel_HueComponentToolTip", "Hue");

	case EColorGradingComponent::Saturation:
		return LOCTEXT("ColorWheel_SaturationComponentToolTip", "Saturation");

	case EColorGradingComponent::Value:
		return LOCTEXT("ColorWheel_ValueComponentToolTip", "Value");

	case EColorGradingComponent::Luminance:
		return LOCTEXT("ColorWheel_LuminanceComponentToolTip", "Luminance");
	}

	return FText::GetEmpty();
}

} //namespace

#undef LOCTEXT_NAMESPACE