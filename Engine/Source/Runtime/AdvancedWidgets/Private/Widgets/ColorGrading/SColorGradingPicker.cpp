// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/ColorGrading/SColorGradingPicker.h"

#include "Styling/AdvancedWidgetsStyle.h"
#include "Widgets/ColorGrading/SColorGradingWheel.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"


#define LOCTEXT_NAMESPACE "ColorGradingWheel"

namespace UE::ColorGrading
{

SColorGradingPicker::~SColorGradingPicker()
{
}

void SColorGradingPicker::Construct(const FArguments& InArgs)
{
	if (!InArgs._SliderValueMin.IsSet())
	{
		SliderValueMin = TNumericLimits<float>::Lowest();
	}
	else
	{
		SliderValueMin = InArgs._SliderValueMin.GetValue();
	}

	if (!InArgs._SliderValueMax.IsSet())
	{
		SliderValueMax = TNumericLimits<float>::Max();
	}
	else
	{
		SliderValueMax = InArgs._SliderValueMax.GetValue();
	}

	check(SliderValueMin < SliderValueMax);

	MainDelta = InArgs._MainDelta;
	MainShiftMultiplier = InArgs._MainShiftMultiplier;
	MainCtrlMultiplier = InArgs._MainCtrlMultiplier;
	ColorGradingModes = InArgs._ColorGradingModes;
	OnColorCommitted = InArgs._OnColorCommitted;
	OnQueryCurrentColor = InArgs._OnQueryCurrentColor;

	float ColorGradingWheelExponent = 2.4f;
	if (ColorGradingModes == EColorGradingModes::Offset)
	{
		ColorGradingWheelExponent = 3.0f;
	}

	ExternalBeginSliderMovementDelegate = InArgs._OnBeginSliderMovement;
	ExternalEndSliderMovementDelegate = InArgs._OnEndSliderMovement;
	ExternalBeginMouseCaptureDelegate = InArgs._OnBeginMouseCapture;
	ExternalEndMouseCaptureDelegate = InArgs._OnEndMouseCapture;

	const auto& AdvancedWidgetsStyle = UE::AdvancedWidgets::FAdvancedWidgetsStyle::Get();

	ChildSlot
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.0)
				.HAlign(HAlign_Fill)
				[
					SNew(SBox)
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.MinAspectRatio(1.0f)
						.MaxAspectRatio(1.0f)
						[
							SNew(SColorGradingWheel)
								.DesiredWheelSize(InArgs._DesiredWheelSize)
								.SelectedColor(TAttribute<FLinearColor>::Create(TAttribute<FLinearColor>::FGetter::CreateSP(this, &SColorGradingPicker::GetCurrentLinearColor)))
								.ExponentDisplacement(ColorGradingWheelExponent)
								.OnValueChanged(this, &SColorGradingPicker::HandleCurrentColorValueChanged, false)
								.OnMouseCaptureBegin(this, &SColorGradingPicker::HandleColorWheelMouseCaptureBegin)
								.OnMouseCaptureEnd(this, &SColorGradingPicker::HandleColorWheelMouseCaptureEnd)
						]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(SBox)
						.HAlign(HAlign_Fill)
						.MaxDesiredWidth(400.f)
						.MinDesiredWidth(400.f)
						[
							SAssignNew(NumericEntryBoxWidget, SNumericEntryBox<float>)
								.EditableTextBoxStyle(&AdvancedWidgetsStyle.GetWidgetStyle<FEditableTextBoxStyle>("ColorGradingPicker.NumericEntry.TextBox"))
								.Font(AdvancedWidgetsStyle.GetFontStyle("ColorGrading.NormalFont"))
								.Value(this, &SColorGradingPicker::OnGetMainValue)
								.OnValueCommitted(this, &SColorGradingPicker::OnMainValueCommitted)
								.OnValueChanged(this, &SColorGradingPicker::OnMainValueChanged, false)
								.AllowSpin(InArgs._AllowSpin.Get())
								.SupportDynamicSliderMaxValue(InArgs._SupportDynamicSliderMaxValue)
								.SupportDynamicSliderMinValue(InArgs._SupportDynamicSliderMinValue)
								.OnDynamicSliderMaxValueChanged(this, &SColorGradingPicker::OnDynamicSliderMaxValueChanged)
								.OnDynamicSliderMinValueChanged(this, &SColorGradingPicker::OnDynamicSliderMinValueChanged)
								.MinValue(InArgs._ValueMin)
								.MaxValue(InArgs._ValueMax)
								.MinSliderValue(SliderValueMin)
								.MaxSliderValue(SliderValueMax)
								.Delta(MainDelta)
								.ShiftMultiplier(MainShiftMultiplier)
								.CtrlMultiplier(MainCtrlMultiplier)
								.OnBeginSliderMovement(this, &SColorGradingPicker::OnBeginSliderMovement)
								.OnEndSliderMovement(this, &SColorGradingPicker::OnEndSliderMovement)
								.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
								.IsEnabled(this, &SColorGradingPicker::IsEntryBoxEnabled)
						]
				]
		];
}

bool SColorGradingPicker::IsEntryBoxEnabled() const
{
	return OnGetMainValue() != TOptional<float>();
}

void SColorGradingPicker::HandleColorWheelMouseCaptureBegin(const FLinearColor& InValue)
{
	bIsMouseDragging = true;
	ExternalBeginMouseCaptureDelegate.ExecuteIfBound();
}

void SColorGradingPicker::HandleColorWheelMouseCaptureEnd(const FLinearColor& InValue)
{
	bIsMouseDragging = false;
	HandleCurrentColorValueChanged(InValue, false);
	ExternalEndMouseCaptureDelegate.ExecuteIfBound();
}

void SColorGradingPicker::OnBeginSliderMovement()
{
	ExternalBeginSliderMovementDelegate.ExecuteIfBound();
	bIsMouseDragging = true;
	//Keep the current value so we can always keep the ratio during the whole drag.

	if (OnQueryCurrentColor.IsBound())
	{
		if (OnQueryCurrentColor.Execute(StartDragRatio))
		{
			TransformColorGradingRangeToLinearColorRange(StartDragRatio);
			float MaxCurrentValue = FMath::Max3(StartDragRatio.X, StartDragRatio.Y, StartDragRatio.Z);
			FVector4 RatioValue(1.0f, 1.0f, 1.0f, 1.0f);
			if (MaxCurrentValue > SMALL_NUMBER)
			{
				RatioValue.X = StartDragRatio.X / MaxCurrentValue;
				RatioValue.Y = StartDragRatio.Y / MaxCurrentValue;
				RatioValue.Z = StartDragRatio.Z / MaxCurrentValue;
			}
			StartDragRatio = RatioValue;
		}
	}
}

void SColorGradingPicker::OnEndSliderMovement(float NewValue)
{
	bIsMouseDragging = false;
	StartDragRatio.X = 1.0f;
	StartDragRatio.Y = 1.0f;
	StartDragRatio.Z = 1.0f;

	OnMainValueChanged(NewValue, false);

	ExternalEndSliderMovementDelegate.ExecuteIfBound();
}

void SColorGradingPicker::AdjustRatioValue(FVector4& NewValue)
{
	if (!bIsMouseDragging)
	{
		return;
	}
	float MaxCurrentValue = FMath::Max3(NewValue.X, NewValue.Y, NewValue.Z);
	if (MaxCurrentValue > SMALL_NUMBER)
	{
		NewValue.X = StartDragRatio.X * MaxCurrentValue;
		NewValue.Y = StartDragRatio.Y * MaxCurrentValue;
		NewValue.Z = StartDragRatio.Z * MaxCurrentValue;
	}
}

void SColorGradingPicker::OnMainValueChanged(float InValue, bool ShouldCommitValueChanges)
{
	if (bIsMouseDragging || ShouldCommitValueChanges)
	{
		TransformColorGradingRangeToLinearColorRange(InValue);

		FVector4 CurrentValue(0.0f, 0.0f, 0.0f, 0.0f);

		if (OnQueryCurrentColor.IsBound())
		{
			if (OnQueryCurrentColor.Execute(CurrentValue))
			{
				TransformColorGradingRangeToLinearColorRange(CurrentValue);

				//The MainValue is the maximum of any channel value
				float MaxCurrentValue = FMath::Max3(CurrentValue.X, CurrentValue.Y, CurrentValue.Z);
				if (MaxCurrentValue <= SMALL_NUMBER)
				{
					//We need the neutral value for the type of color grading, currently only offset is an addition(0.0) all other are multiplier(1.0)
					CurrentValue = FVector4(InValue, InValue, InValue, CurrentValue.W);
				}
				else
				{
					float Ratio = InValue / MaxCurrentValue;
					CurrentValue *= FVector4(Ratio, Ratio, Ratio, 1.0f);
					AdjustRatioValue(CurrentValue);
				}
				TransformLinearColorRangeToColorGradingRange(CurrentValue);
				OnColorCommitted.ExecuteIfBound(CurrentValue, ShouldCommitValueChanges);
			}
		}
	}
}

void SColorGradingPicker::OnMainValueCommitted(float InValue, ETextCommit::Type CommitType)
{
	OnMainValueChanged(InValue, true);
}

TOptional<float> SColorGradingPicker::OnGetMainValue() const
{
	FVector4 CurrentValue(0.0f, 0.0f, 0.0f, 0.0f);

	if (OnQueryCurrentColor.IsBound())
	{
		if (OnQueryCurrentColor.Execute(CurrentValue))
		{
			//The MainValue is the maximum of any channel value
			TOptional<float> CurrentValueOption = FMath::Max3(CurrentValue.X, CurrentValue.Y, CurrentValue.Z);
			return CurrentValueOption;
		}
	}

	return TOptional<float>();
}

void SColorGradingPicker::TransformLinearColorRangeToColorGradingRange(FVector4& VectorValue) const
{
	float Ratio = (SliderValueMax - SliderValueMin);
	VectorValue *= Ratio;
	VectorValue += FVector4(SliderValueMin, SliderValueMin, SliderValueMin, SliderValueMin);
}

void SColorGradingPicker::TransformColorGradingRangeToLinearColorRange(FVector4& VectorValue) const
{
	VectorValue += -FVector4(SliderValueMin, SliderValueMin, SliderValueMin, SliderValueMin);
	float Ratio = 1.0f / (SliderValueMax - SliderValueMin);
	VectorValue *= Ratio;
}

void SColorGradingPicker::TransformColorGradingRangeToLinearColorRange(float& FloatValue)
{
	FloatValue -= SliderValueMin;
	float Ratio = 1.0f / (SliderValueMax - SliderValueMin);
	FloatValue *= Ratio;
}

FLinearColor SColorGradingPicker::GetCurrentLinearColor()
{
	FLinearColor CurrentColor = FLinearColor::Black;
	FVector4 CurrentValue;

	if (OnQueryCurrentColor.IsBound())
	{
		if (OnQueryCurrentColor.Execute(CurrentValue))
		{
			TransformColorGradingRangeToLinearColorRange(CurrentValue);
			CurrentColor = FLinearColor(CurrentValue.X, CurrentValue.Y, CurrentValue.Z);
		}
	}

	return CurrentColor.LinearRGBToHSV();
}

void SColorGradingPicker::HandleCurrentColorValueChanged(const FLinearColor& NewValue, bool ShouldCommitValueChanges)
{
	//Query the current vector4 so we can pass back the w value
	FVector4 CurrentValue(0.0f, 0.0f, 0.0f, 0.0f);
	if (OnQueryCurrentColor.IsBound())
	{
		if (OnQueryCurrentColor.Execute(CurrentValue))
		{
			FLinearColor NewValueRGB = NewValue.HSVToLinearRGB();
			FVector4 NewValueVector(NewValueRGB.R, NewValueRGB.G, NewValueRGB.B);
			TransformLinearColorRangeToColorGradingRange(NewValueVector);
			//Set the W with the original value
			NewValueVector.W = CurrentValue.W;
			OnColorCommitted.ExecuteIfBound(NewValueVector, ShouldCommitValueChanges);
		}
	}
}

void SColorGradingPicker::OnDynamicSliderMaxValueChanged(float NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfHigher)
{
	TSharedPtr<SSpinBox<float>> SpinBox = StaticCastSharedPtr<SSpinBox<float>>(NumericEntryBoxWidget->GetSpinBox());

	if (SpinBox.IsValid())
	{
		if (SpinBox != InValueChangedSourceWidget)
		{
			if ((NewMaxSliderValue > SpinBox->GetMaxSliderValue() && UpdateOnlyIfHigher) || !UpdateOnlyIfHigher)
			{
				SpinBox->SetMaxSliderValue(NewMaxSliderValue);
				SliderValueMax = NewMaxSliderValue;
			}
		}
	}

	if (IsOriginator)
	{
		OnNumericEntryBoxDynamicSliderMaxValueChanged.Broadcast(NewMaxSliderValue, InValueChangedSourceWidget, false, UpdateOnlyIfHigher);
	}
}

void SColorGradingPicker::OnDynamicSliderMinValueChanged(float NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfLower)
{
	TSharedPtr<SSpinBox<float>> SpinBox = StaticCastSharedPtr<SSpinBox<float>>(NumericEntryBoxWidget->GetSpinBox());

	if (SpinBox.IsValid())
	{
		if (SpinBox != InValueChangedSourceWidget)
		{
			if ((NewMinSliderValue < SpinBox->GetMinSliderValue() && UpdateOnlyIfLower) || !UpdateOnlyIfLower)
			{
				SpinBox->SetMinSliderValue(NewMinSliderValue);
				SliderValueMin = NewMinSliderValue;
			}
		}
	}

	if (IsOriginator)
	{
		OnNumericEntryBoxDynamicSliderMinValueChanged.Broadcast(NewMinSliderValue, InValueChangedSourceWidget, false, UpdateOnlyIfLower);
	}
}

} //namespace

#undef LOCTEXT_NAMESPACE
