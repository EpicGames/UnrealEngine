// Copyright Epic Games, Inc. All Rights Reserved.
#include "Widgets/ColorGrading/SColorGradingWheel.h"

#include "Materials/Material.h"
#include "SlateMaterialBrush.h"
#include "Styling/CoreStyle.h"
#include "Styling/AdvancedWidgetsStyle.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/DrawElements.h"

namespace UE::ColorGrading
{

SLATE_IMPLEMENT_WIDGET(SColorGradingWheel)
void SColorGradingWheel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "SelectedColor", SelectedColorAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "DesiredWheelSize", DesiredWheelSizeAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ExponentDisplacement", ExponentDisplacementAttribute, EInvalidateWidgetReason::Paint);
}

SColorGradingWheel::SColorGradingWheel()
	: SelectedColorAttribute(*this, FLinearColor(ForceInit))
	, DesiredWheelSizeAttribute(*this)
	, ExponentDisplacementAttribute(*this)
	, Union_IsAttributeSet(0)
{
}

/* SColorGradingWheel methods
 *****************************************************************************/

void SColorGradingWheel::Construct(const FArguments& InArgs)
{
	// Create the brush here since we need to dynamically load the material for it
	BackgroundMaterial = LoadObject<UMaterial>(nullptr, TEXT("Material'/Engine/EngineMaterials/ColorGradingWheel.ColorGradingWheel'"));
	check(BackgroundMaterial);
	BackgroundImage = MakeShared<FSlateMaterialBrush>(*BackgroundMaterial.Get(), FVector2f(400.f, 400.f));

	CrossImage = UE::AdvancedWidgets::FAdvancedWidgetsStyle::Get().GetBrush("ColorGradingWheel.Cross");
	SelectorImage = UE::AdvancedWidgets::FAdvancedWidgetsStyle::Get().GetBrush("ColorGradingWheel.Selector");
	SetSelectedColorAttribute(InArgs._SelectedColor);
	SetDesiredWheelSizeAttribute(InArgs._DesiredWheelSize);
	SetExponentDisplacementAttribute(InArgs._ExponentDisplacement);
	OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;
	OnValueChanged = InArgs._OnValueChanged;
}

void SColorGradingWheel::SetSelectedColorAttribute(TAttribute<FLinearColor> InSelectedColor)
{
	SelectedColorAttribute.Assign(*this, MoveTemp(InSelectedColor));
}

void SColorGradingWheel::SetDesiredWheelSizeAttribute(TAttribute<int32> InDesiredWheelSize)
{
	const bool bDesiredWheelSizeSetChanged = (bIsAttributeDesiredWheelSizeSet != InDesiredWheelSize.IsSet());
	bIsAttributeDesiredWheelSizeSet = InDesiredWheelSize.IsSet();
	const bool bAttributeWasAssgined = DesiredWheelSizeAttribute.Assign(*this, MoveTemp(InDesiredWheelSize));

	// If the assign didn't invalidate the widget but the attribute set changed, then invalidate the widget.
	if (bDesiredWheelSizeSetChanged && !bAttributeWasAssgined)
	{
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SColorGradingWheel::SetExponentDisplacementAttribute(TAttribute<float> InExponentDisplacement)
{
	const bool bExponentDisplacementSetChanged = (bIsAttributeExponentDisplacementSet != InExponentDisplacement.IsSet());
	bIsAttributeExponentDisplacementSet = InExponentDisplacement.IsSet();
	const bool bAttributeWasAssgined = ExponentDisplacementAttribute.Assign(*this, MoveTemp(InExponentDisplacement), 1.f);

	// If the assign didn't invalidate the widget but the attribute set changed, then invalidate the widget.
	if (bExponentDisplacementSetChanged && !bAttributeWasAssgined)
	{
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

FVector2f SColorGradingWheel::GetActualSize(const FGeometry& MyGeometry) const
{
	const FVector2f AllottedGeometrySize = MyGeometry.GetLocalSize();

	if (bIsAttributeDesiredWheelSizeSet)
	{
		// Even if a desired size is provided, make sure the wheel is painted within the allotted geometry
		int32 CachedDesiredWheelSize = DesiredWheelSizeAttribute.Get();
		float ActualSize = FMath::Min(CachedDesiredWheelSize, AllottedGeometrySize.GetMin());

		return FVector2f(ActualSize, ActualSize);
	}

	return AllottedGeometrySize;
}

FString SColorGradingWheel::GetReferencerName() const
{
	return TEXT("SColorGradingWheel");
}

void SColorGradingWheel::AddReferencedObjects(FReferenceCollector& InCollector)
{
	InCollector.AddReferencedObject(BackgroundMaterial);
}


/* SWidget overrides
 *****************************************************************************/

FVector2D SColorGradingWheel::ComputeDesiredSize(float) const
{
	if (bIsAttributeDesiredWheelSizeSet)
	{
		int32 CachedDesiredWheelSize = DesiredWheelSizeAttribute.Get();
		return FVector2D(CachedDesiredWheelSize, CachedDesiredWheelSize);
	}
	return FVector2D(BackgroundImage->ImageSize);
}


FReply SColorGradingWheel::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return FReply::Handled();
}


FReply SColorGradingWheel::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (OnMouseCaptureBegin.IsBound())
		{
			SelectedColorAttribute.UpdateNow(*this);
			OnMouseCaptureBegin.Execute(SelectedColorAttribute.Get());
		}

		if (!ProcessMouseAction(MyGeometry, MouseEvent, false))
		{
			if (OnMouseCaptureEnd.IsBound())
			{
				SelectedColorAttribute.UpdateNow(*this);
				OnMouseCaptureEnd.Execute(SelectedColorAttribute.Get());
			}
			return FReply::Unhandled();
		}

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}


FReply SColorGradingWheel::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && HasMouseCapture())
	{
		if (OnMouseCaptureEnd.IsBound())
		{
			SelectedColorAttribute.UpdateNow(*this);
			OnMouseCaptureEnd.Execute(SelectedColorAttribute.Get());
		}

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}


FReply SColorGradingWheel::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!HasMouseCapture())
	{
		return FReply::Unhandled();
	}

	ProcessMouseAction(MyGeometry, MouseEvent, true);

	return FReply::Handled();
}


int32 SColorGradingWheel::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	FVector2f GeometrySize = GetActualSize(AllottedGeometry);

	const FVector2f SelectorSize = SelectorImage->ImageSize;
	const FVector2f CircleSize = GeometrySize - SelectorSize;

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(CircleSize, FSlateLayoutTransform(0.5f * SelectorSize)),
		BackgroundImage.Get(),
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * BackgroundImage->GetTint(InWidgetStyle)
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 1,
		AllottedGeometry.ToPaintGeometry(CircleSize, FSlateLayoutTransform(0.5f * SelectorSize)),
		CrossImage,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * CrossImage->GetTint(InWidgetStyle)
	);

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId + 2,
		AllottedGeometry.ToPaintGeometry(SelectorSize, FSlateLayoutTransform(0.5f * (GeometrySize + CalcRelativePositionFromCenter() * CircleSize - SelectorSize))),
		SelectorImage,
		DrawEffects,
		InWidgetStyle.GetColorAndOpacityTint() * SelectorImage->GetTint(InWidgetStyle)
	);

	return LayerId + 1;
}


/* SColorGradingWheel implementation
 *****************************************************************************/

UE::Slate::FDeprecateVector2DResult SColorGradingWheel::CalcRelativePositionFromCenter() const
{
	float Hue = SelectedColorAttribute.Get().R;
	float Saturation = SelectedColorAttribute.Get().G;
	if (bIsAttributeExponentDisplacementSet && ExponentDisplacementAttribute.Get() != 1.0f && !FMath::IsNearlyEqual(ExponentDisplacementAttribute.Get(), 0.0f, 0.00001f))
	{
		//Use log curve to set the distance G value
		Saturation = FMath::Pow(Saturation, 1.0f / ExponentDisplacementAttribute.Get());
	}
	float Angle = HueAngleOffset - (Hue / 180.0f * PI);
	float Radius = Saturation;

	return FVector2f(FMath::Cos(Angle), FMath::Sin(Angle)) * Radius;
}


bool SColorGradingWheel::ProcessMouseAction(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bProcessWhenOutsideColorWheel)
{
	if (bIsAttributeDesiredWheelSizeSet)
	{
		DesiredWheelSizeAttribute.UpdateNow(*this);
	}

	const FVector2f GeometrySize = GetActualSize(MyGeometry);

	const FVector2f LocalMouseCoordinate = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	const FVector2f RelativePositionFromCenter = (2.0f * LocalMouseCoordinate - GeometrySize) / (GeometrySize - SelectorImage->ImageSize);
	const float RelativeRadius = RelativePositionFromCenter.Size();

	if (RelativeRadius <= 1.0f || bProcessWhenOutsideColorWheel)
	{
		float Angle = FMath::Atan2(-RelativePositionFromCenter.Y, RelativePositionFromCenter.X) + HueAngleOffset;

		if (Angle < 0.0f)
		{
			Angle += 2.0f * PI;
		}

		SelectedColorAttribute.UpdateNow(*this);
		FLinearColor NewColor = SelectedColorAttribute.Get();
		{
			NewColor.R = Angle * 180.0f * INV_PI;
			float LinearRadius = FMath::Min(RelativeRadius, 1.0f);
			if (bIsAttributeExponentDisplacementSet && ExponentDisplacementAttribute.Get() != 1.0f)
			{
				//Use log curve to set the distance G value
				float LogDistance = FMath::Pow(LinearRadius, ExponentDisplacementAttribute.Get());
				NewColor.G = LogDistance;
			}
			else
			{
				NewColor.G = LinearRadius;
			}
		}

		OnValueChanged.ExecuteIfBound(NewColor);
	}

	return (RelativeRadius <= 1.0f);
}

} //namespace
