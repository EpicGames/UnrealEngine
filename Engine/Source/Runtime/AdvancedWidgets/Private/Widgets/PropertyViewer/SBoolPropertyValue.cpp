// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PropertyViewer/SBoolPropertyValue.h"

#include "Framework/PropertyViewer/INotifyHook.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

#include "Widgets/Input/SCheckBox.h"

namespace UE::PropertyViewer
{

TSharedPtr<SWidget> SBoolPropertyValue::CreateInstance(const FPropertyValueFactory::FGenerateArgs Args)
{
	return SNew(SBoolPropertyValue)
		.Path(Args.Path)
		.NotifyHook(Args.NotifyHook)
		.IsEnabled(Args.bCanEditValue);
}


void SBoolPropertyValue::Construct(const FArguments& InArgs)
{
	Path = InArgs._Path;
	NotifyHook = InArgs._NotifyHook;

	const FProperty* Property = Path.GetLastProperty();
	if (CastField<const FBoolProperty>(Property) && Property->ArrayDim == 1)
	{
		ChildSlot
		[
			SNew(SCheckBox)
			.IsChecked(this, &SBoolPropertyValue::HandleIsChecked)
			.OnCheckStateChanged(this, &SBoolPropertyValue::HandleCheckStateChanged)
 		];
	}
}


ECheckBoxState SBoolPropertyValue::HandleIsChecked() const
{
	if (const void* Container = Path.GetContainerPtr())
	{
		if (const FBoolProperty* Property = CastField<const FBoolProperty>(Path.GetLastProperty()))
		{
			const bool bValue = Property->GetPropertyValue_InContainer(Container);
			return bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}
	return ECheckBoxState::Undetermined;
}


void SBoolPropertyValue::HandleCheckStateChanged(ECheckBoxState NewState)
{
	if (void* Container = Path.GetContainerPtr())
	{
		if (const FBoolProperty* Property = CastField<const FBoolProperty>(Path.GetLastProperty()))
		{
			if (NotifyHook)
			{
				NotifyHook->OnPreValueChange(Path);
			}
			bool bValue = NewState == ECheckBoxState::Checked;
			Property->SetPropertyValue_InContainer(Container, bValue);
			if (NotifyHook)
			{
				NotifyHook->OnPostValueChange(Path);
			}
		}
	}
}

} //namespace
