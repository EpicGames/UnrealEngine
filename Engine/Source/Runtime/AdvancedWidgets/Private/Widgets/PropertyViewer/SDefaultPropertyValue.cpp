// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PropertyViewer/SDefaultPropertyValue.h"

#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

#include "Widgets/Text/STextBlock.h"

namespace UE::PropertyViewer
{

TSharedPtr<SWidget> SDefaultPropertyValue::CreateInstance(const FPropertyValueFactory::FGenerateArgs Args)
{
	return SNew(SDefaultPropertyValue)
		.Path(Args.Path)
		.IsEnabled(Args.bCanEditValue);
}


void SDefaultPropertyValue::Construct(const FArguments& InArgs)
{
	Path = InArgs._Path;

	const FProperty* Property = Path.GetLastProperty();
	ChildSlot
	[
		SNew(STextBlock)
		.Text(this, &SDefaultPropertyValue::GetText)
 	];
}


FText SDefaultPropertyValue::GetText() const
{
	if (const void* ContainerPtr = Path.GetContainerPtr())
	{
		if (const FProperty* LastProperty = Path.GetLastProperty())
		{
			FString TextVersion;
			LastProperty->ExportTextItem_InContainer(TextVersion, ContainerPtr, nullptr, nullptr, PPF_SimpleObjectText);
			return FText::FromString(TextVersion);
		}
	}
	return FText::GetEmpty();
}

} //namespace
