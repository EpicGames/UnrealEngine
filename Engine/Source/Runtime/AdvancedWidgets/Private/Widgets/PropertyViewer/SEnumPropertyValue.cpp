// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PropertyViewer/SEnumPropertyValue.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/PropertyViewer/INotifyHook.h"
#include "Styling/SlateTypes.h"
#include "Styling/AdvancedWidgetsStyle.h"
#include "Styling/AppStyle.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::PropertyViewer
{

TSharedPtr<SWidget> SEnumPropertyValue::CreateInstance(const FPropertyValueFactory::FGenerateArgs Args)
{
	return SNew(SEnumPropertyValue)
		.Path(Args.Path)
		.NotifyHook(Args.NotifyHook)
		.IsEnabled(Args.bCanEditValue);
}


void SEnumPropertyValue::Construct(const FArguments& InArgs)
{
	Path = InArgs._Path;
	NotifyHook = InArgs._NotifyHook;

	const FProperty* Property = Path.GetLastProperty();
	const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Property);
	const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(Property);
	const UEnum* Enum = EnumProperty ? EnumProperty->GetEnum() : NumericProperty ? NumericProperty->GetIntPropertyEnum() : nullptr;
	EnumType = Enum;

	if (Enum && Property->ArrayDim == 1)
	{
		ChildSlot
		[
			SNew(SComboButton)
			.ComboButtonStyle(&::UE::AdvancedWidgets::FAdvancedWidgetsStyle::Get().GetWidgetStyle<FComboButtonStyle>("PropertyValue.ComboButton"))
			.ContentPadding(FMargin(0.f))
			.HAlign(HAlign_Left)
			.OnGetMenuContent(this, &SEnumPropertyValue::OnGetMenuContent)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &SEnumPropertyValue::GetText)
			]
 		];
	}
}


FText SEnumPropertyValue::GetText() const
{
	if (const UEnum* EnumPtr = EnumType.Get())
	{
		int64 CurrentValue = GetCurrentValue();
		return EnumPtr->GetDisplayNameTextByValue(CurrentValue);
	}

	return FText::GetEmpty();
}


int32 SEnumPropertyValue::GetCurrentValue() const
{
	if (const UEnum* EnumPtr = EnumType.Get())
	{
		if (const void* Container = Path.GetContainerPtr())
		{
			if (const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(Path.GetLastProperty()))
			{
					return NumericProperty->GetSignedIntPropertyValue(NumericProperty->ContainerPtrToValuePtr<const void*>(Container));
			}
			else if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Path.GetLastProperty()))
			{
				return EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(EnumProperty->ContainerPtrToValuePtr<const void*>(Container));
			}
		}
	}
	return 0;
}


TSharedRef<SWidget> SEnumPropertyValue::OnGetMenuContent()
{
	const bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr);

	if (const UEnum* EnumPtr = EnumType.Get())
	{
		// bitflag is not supported at runtime (WITH_EDITORONLY_DATA)
		const bool bHasMaxValue = EnumPtr->ContainsExistingMax();
		const int32 NumEnums = bHasMaxValue ? EnumPtr->NumEnums() - 1 : EnumPtr->NumEnums();

		for (int32 Index = 0; Index < NumEnums; ++Index)
		{
#if WITH_EDITORONLY_DATA
			if (!EnumPtr->HasMetaData(TEXT("Hidden"), Index))
#endif
			{
#if WITH_EDITOR
				FText Tooltip = EnumPtr->GetToolTipTextByIndex(Index);
#else
				FText Tooltip = FText::GetEmpty();
#endif

				MenuBuilder.AddMenuEntry(
					EnumPtr->GetDisplayNameTextByIndex(Index),
					Tooltip,
					FSlateIcon(),
					FUIAction
					(
						FExecuteAction::CreateSP(this, &SEnumPropertyValue::SetEnumEntry, Index),
						FCanExecuteAction(),
						//FIsActionChecked::CreateSP(this, &SEnumPropertyValue::IsEnumEntryChecked, Index)
						FIsActionChecked()
					),
					NAME_None,
					EUserInterfaceActionType::None);
			}
		}
	}

	return MenuBuilder.MakeWidget();
}


void SEnumPropertyValue::SetEnumEntry(int32 Index)
{
	if (const UEnum* EnumPtr = EnumType.Get())
	{
		if (void* Container = Path.GetContainerPtr())
		{
			int64 NewValue = EnumPtr->GetValueByIndex(Index);
			if (const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(Path.GetLastProperty()))
			{
				if (NotifyHook)
				{
					NotifyHook->OnPreValueChange(Path);
				}
				NumericProperty->SetIntPropertyValue(NumericProperty->ContainerPtrToValuePtr<const void*>(Container), NewValue);
				if (NotifyHook)
				{
					NotifyHook->OnPostValueChange(Path);
				}
			}
			else if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(Path.GetLastProperty()))
			{
				if (NotifyHook)
				{
					NotifyHook->OnPreValueChange(Path);
				}
				EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(EnumProperty->ContainerPtrToValuePtr<const void*>(Container), NewValue);
				if (NotifyHook)
				{
					NotifyHook->OnPostValueChange(Path);
				}
			}
		}
	}
}


//bool SEnumPropertyValue::IsEnumEntryChecked(int32 Index) const
//{
//	if (UEnum* EnumPtr = EnumType.Get())
//	{
//		return (GetCurrentValue() & EnumPtr->GetValueByIndex(Index)) != 0;
//	}
//	return false;
//}

} //namespace
