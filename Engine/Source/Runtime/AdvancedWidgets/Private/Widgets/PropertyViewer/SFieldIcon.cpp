// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PropertyViewer/SFieldIcon.h"

#include "Framework/PropertyViewer/FieldIconFinder.h"
#include "Widgets/Images/SLayeredImage.h"


namespace UE::PropertyViewer
{


void SFieldIcon::Construct(const FArguments& InArgs, const UClass* Class)
{
	check(Class);

	ChildSlot
	[
		SNew(SImage)
		.Image(FFieldIconFinder::GetIcon(Class))
	];
}


void SFieldIcon::Construct(const FArguments& InArgs, const UScriptStruct* Struct)
{
	check(Struct);

	ChildSlot
	[
		SNew(SImage)
		.Image(FFieldIconFinder::GetIcon(Struct))
	];
}


void SFieldIcon::Construct(const FArguments& InArgs, const FProperty* Property)
{
	check(Property);

	FFieldIconFinder::FFieldIconArray Icons = InArgs._OverrideColorSettings.IsSet() ? FFieldIconFinder::GetPropertyIcon(Property, InArgs._OverrideColorSettings.GetValue()) : FFieldIconFinder::GetPropertyIcon(Property);
	if (Icons.Num() == 1)
	{
		ChildSlot
		[
			SNew(SImage)
			.Image(Icons[0].Icon)
			.ColorAndOpacity(Icons[0].Color)
		];
	}
	else
	{
		TArray<SLayeredImage::ImageLayer> Layers;
		Layers.Reserve(Icons.Num());
		for (const FFieldIconFinder::FFieldIcon& Icon : Icons)
		{
			Layers.Emplace(Icon.Icon, Icon.Color);
		}

		ChildSlot
		[
			SNew(SLayeredImage, MoveTemp(Layers))
		];
	}
}


void SFieldIcon::Construct(const FArguments& InArgs, const UFunction* Function)
{
	check(Function);

	FFieldIconFinder::FFieldIconArray Icons = InArgs._OverrideColorSettings.IsSet() ? FFieldIconFinder::GetFunctionIcon(Function, InArgs._OverrideColorSettings.GetValue()) : FFieldIconFinder::GetFunctionIcon(Function);
	if (Icons.Num() >= 1)
	{
		ChildSlot
		[
			SNew(SImage)
			.Image(Icons[0].Icon)
			.ColorAndOpacity(Icons[0].Color)
		];
	}
}


} //namespace
