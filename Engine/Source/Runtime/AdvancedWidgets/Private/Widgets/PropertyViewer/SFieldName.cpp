// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PropertyViewer/SFieldName.h"

#include "Application/SlateApplicationBase.h"
#include "UObject/UnrealType.h"
#include "UObject/Class.h"
#include "Widgets/PropertyViewer/SFieldIcon.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"


#define LOCTEXT_NAMESPACE "SFieldName"

namespace UE::PropertyViewer
{

void SFieldName::Construct(const FArguments& InArgs, const UClass* Class)
{
	check(Class);

	TSharedPtr<SWidget> Icon;
	if (InArgs._bShowIcon)
	{
		Icon = SNew(SFieldIcon, Class)
			.OverrideColorSettings(InArgs._OverrideIconColorSettings);
	}

	FText DisplayName;
	if (InArgs._OverrideDisplayName.IsSet())
	{
		DisplayName = InArgs._OverrideDisplayName.GetValue();
	}
	else
	{
#if WITH_EDITORONLY_DATA
		DisplayName = InArgs._bSanitizeName ? Class->GetDisplayNameText() : FText::FromName(Class->GetFName());
#else
		DisplayName = FText::FromName(Class->GetFName());
#endif
	}

	Field = Class;
	Construct(InArgs, DisplayName, Icon);
}


void SFieldName::Construct(const FArguments& InArgs, const UScriptStruct* Struct)
{
	check(Struct);

	TSharedPtr<SWidget> Icon;
	if (InArgs._bShowIcon)
	{
		Icon = SNew(SFieldIcon, Struct)
			.OverrideColorSettings(InArgs._OverrideIconColorSettings);
	}

	FText DisplayName;
	if (InArgs._OverrideDisplayName.IsSet())
	{
		DisplayName = InArgs._OverrideDisplayName.GetValue();
	}
	else
	{
#if WITH_EDITORONLY_DATA
		DisplayName = InArgs._bSanitizeName ? Struct->GetDisplayNameText() : FText::FromName(Struct->GetFName());
#else
		DisplayName = FText::FromName(Struct->GetFName());
#endif
	}

	Field = Struct;
	Construct(InArgs, DisplayName, Icon);
}


void SFieldName::Construct(const FArguments& InArgs, const FProperty* Property)
{
	check(Property);

	TSharedPtr<SWidget> Icon;
	if (InArgs._bShowIcon)
	{
		Icon = SNew(SFieldIcon, Property)
			.OverrideColorSettings(InArgs._OverrideIconColorSettings);
	}
	
	FText DisplayName;
	if (InArgs._OverrideDisplayName.IsSet())
	{
		DisplayName = InArgs._OverrideDisplayName.GetValue();
	}
	else
	{
#if WITH_EDITORONLY_DATA
		DisplayName = InArgs._bSanitizeName ? Property->GetDisplayNameText() : FText::FromName(Property->GetFName());
#else
		DisplayName = FText::FromName(Property->GetFName());
#endif
	}

	Field = Property;
	Construct(InArgs, DisplayName, Icon);
}


void SFieldName::Construct(const FArguments& InArgs, const UFunction* Function)
{
	check(Function);

	TSharedPtr<SWidget> Icon;
	if (InArgs._bShowIcon)
	{
		Icon = SNew(SFieldIcon, Function)
			.OverrideColorSettings(InArgs._OverrideIconColorSettings);
	}

	FText DisplayName;
	if (InArgs._OverrideDisplayName.IsSet())
	{
		DisplayName = InArgs._OverrideDisplayName.GetValue();
	}
	else
	{
#if WITH_EDITORONLY_DATA
		DisplayName = InArgs._bSanitizeName ? Function->GetDisplayNameText() : FText::FromName(Function->GetFName());
#else
		DisplayName = FText::FromName(Function->GetFName());
#endif
	}

	Field = Function;
	Construct(InArgs, DisplayName, Icon);
}


void SFieldName::Construct(const FArguments& InArgs, const FText& DisplayName, TSharedPtr<SWidget> Icon)
{
	if (Icon)
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			[
				Icon.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.Padding(4.0f)
			[
				SAssignNew(NameBlock, STextBlock)
				.Text(DisplayName)
				.HighlightText(InArgs._HighlightText)
				.ToolTipText(GetToolTipText())
			]
		];
	}
	else
	{
		ChildSlot
		[
			SAssignNew(NameBlock, STextBlock)
			.Text(DisplayName)
			.HighlightText(InArgs._HighlightText)
			.ToolTipText(GetToolTipText())
		];
	}
}


void SFieldName::SetHighlightText(TAttribute<FText> InHighlightText)
{
	if (NameBlock)
	{
		NameBlock->SetHighlightText(MoveTemp(InHighlightText));
	}
}

FText SFieldName::GetToolTipText() const
{
#if WITH_EDITORONLY_DATA
	if (FProperty* PropertyPtr = Field.Get<FProperty>())
	{
		return FText::FormatOrdered(LOCTEXT("PropertyTooltip", "{0}\nProperty: {1} {2}")
			, PropertyPtr->GetToolTipText()
			, FText::FromString(PropertyPtr->GetCPPType())
			, FText::FromString(PropertyPtr->GetName()));
	}
	if (UFunction* FunctionPtr = Field.Get<UFunction>())
	{
		const FProperty* ReturnType = FunctionPtr->GetReturnProperty();
		return FText::FormatOrdered(LOCTEXT("FunctionTooltip", "{0}\nFunction: {1}\nReturns: {2}")
			, FunctionPtr->GetToolTipText()
			, FText::FromString(FunctionPtr->GetName())
			, (ReturnType ? FText::FromString(ReturnType->GetCPPType()) : FText::GetEmpty()));
	}
	if (UField* FieldPtr = Field.Get<UField>())
	{
		return FText::FormatOrdered(LOCTEXT("FieldTooltip", "{0}\nType: {1}")
			, FieldPtr->GetToolTipText()
			, FText::FromString(FieldPtr->GetName()));
	}
#endif
	return FText::GetEmpty();
}

} //namespace

#undef LOCTEXT_NAMESPACE