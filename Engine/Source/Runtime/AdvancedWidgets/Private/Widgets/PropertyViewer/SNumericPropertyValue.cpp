// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PropertyViewer/SNumericPropertyValue.h"

#include "Framework/PropertyViewer/INotifyHook.h"
#include "Styling/AdvancedWidgetsStyle.h"
#include "Styling/SlateTypes.h"
#include "UObject/UnrealType.h"

#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "SNumericPropertyValue"

namespace UE::PropertyViewer
{

TSharedPtr<SWidget> SNumericPropertyValue::CreateInstance(const FPropertyValueFactory::FGenerateArgs Args)
{
	return SNew(SNumericPropertyValue)
		.Path(Args.Path)
		.NotifyHook(Args.NotifyHook)
		.IsEnabled(Args.bCanEditValue);
}

void SNumericPropertyValue::Construct(const FArguments& InArgs)
{
	Path = InArgs._Path;
	NotifyHook = InArgs._NotifyHook;

	if (const FNumericProperty* Property = CastField<const FNumericProperty>(Path.GetLastProperty()))
	{
		if (Property->ArrayDim == 1)
		{
			const FSpinBoxStyle& SpinBoxStyle = ::UE::AdvancedWidgets::FAdvancedWidgetsStyle::Get().GetWidgetStyle<FSpinBoxStyle>("PropertyValue.SpinBox");
			// int8/int16/int32/int64
			if (Property->CanHoldValue(-1) && Property->IsInteger())
			{
				ChildSlot
				[
					SNew(SSpinBox<int64>)
					.Style(&SpinBoxStyle)
					.Value(this, &SNumericPropertyValue::GetCurrentValue_int64)
					.OnValueChanged(this, &SNumericPropertyValue::OnValueChanged_int64)
					.OnValueCommitted(this, &SNumericPropertyValue::OnValueCommitted_int64)
					.OnEndSliderMovement(this, &SNumericPropertyValue::OnEndSliderMovement_int64)
				];
			}
			// uint8/uint16/uint32/uint64
			else if (Property->IsInteger())
			{
				ChildSlot
				[
					SNew(SSpinBox<uint64>)
					.Style(&SpinBoxStyle)
					.Value(this, &SNumericPropertyValue::GetCurrentValue_uint64)
					.OnValueChanged(this, &SNumericPropertyValue::OnValueChanged_uint64)
					.OnValueCommitted(this, &SNumericPropertyValue::OnValueCommitted_uint64)
					.OnEndSliderMovement(this, &SNumericPropertyValue::OnEndSliderMovement_uint64)
				];
			}
			// float/double
			else
			{
				ChildSlot
				[
					SNew(SSpinBox<double>)
					.Style(&SpinBoxStyle)
					.Value(this, &SNumericPropertyValue::GetCurrentValue_double)
					.OnValueChanged(this, &SNumericPropertyValue::OnValueChanged_double)
					.OnValueCommitted(this, &SNumericPropertyValue::OnValueCommitted_double)
					.OnEndSliderMovement(this, &SNumericPropertyValue::OnEndSliderMovement_double)
				];
			}
		}
	}
}

void SNumericPropertyValue::OnEndSliderMovement_uint64(uint64 Value)
{
	OnValueChanged_uint64(Value);
}
void SNumericPropertyValue::OnEndSliderMovement_int64(int64 Value)
{
	OnValueChanged_int64(Value);
}
void SNumericPropertyValue::OnEndSliderMovement_double(double Value)
{
	OnValueChanged_double(Value);
}


namespace Private
{
	template<typename TValue, typename TPred>
	TValue GetCurrentValue(const FPropertyPath& Path, TPred Pred)
	{
		if (const void* Container = Path.GetContainerPtr())
		{
			if (const FNumericProperty* Property = CastField<const FNumericProperty>(Path.GetLastProperty()))
			{
				return Pred(Property, Property->ContainerPtrToValuePtr<const void*>(Container));
			}
		}
		return TValue();
	}
}// namespace


uint64 SNumericPropertyValue::GetCurrentValue_uint64() const
{
	return Private::GetCurrentValue<uint64>(Path, [](const FNumericProperty* Property, const void* DataPtr) { return Property->GetUnsignedIntPropertyValue(DataPtr); });
}
int64 SNumericPropertyValue::GetCurrentValue_int64() const
{
	return Private::GetCurrentValue<int64>(Path, [](const FNumericProperty* Property, const void* DataPtr) { return Property->GetSignedIntPropertyValue(DataPtr); });
}
double SNumericPropertyValue::GetCurrentValue_double() const
{
	return Private::GetCurrentValue<double>(Path, [](const FNumericProperty* Property, const void* DataPtr) { return Property->GetFloatingPointPropertyValue(DataPtr); });
}


namespace Private
{
	template<typename TPred>
	void OnValueChanged(FPropertyPath& Path, INotifyHook* NotifyHook, TPred Pred)
	{
		if (void* Container = Path.GetContainerPtr())
		{
			if (const FNumericProperty* Property = CastField<const FNumericProperty>(Path.GetLastProperty()))
			{
				if (NotifyHook)
				{
					NotifyHook->OnPreValueChange(Path);
				}
				Pred(Property, Property->ContainerPtrToValuePtr<void*>(Container));
				if (NotifyHook)
				{
					NotifyHook->OnPostValueChange(Path);
				}
			}
		}
	}
}// namespace

void SNumericPropertyValue::OnValueChanged_uint64(uint64 Value)
{
	Private::OnValueChanged(Path, NotifyHook, [Value](const FNumericProperty* Property, void* DataPtr) { Property->SetIntPropertyValue(DataPtr, Value); });
}
void SNumericPropertyValue::OnValueChanged_int64(int64 Value)
{
	Private::OnValueChanged(Path, NotifyHook, [Value](const FNumericProperty* Property, void* DataPtr) { Property->SetIntPropertyValue(DataPtr, Value); });
}
void SNumericPropertyValue::OnValueChanged_double(double Value)
{
	Private::OnValueChanged(Path, NotifyHook, [Value](const FNumericProperty* Property, void* DataPtr) { Property->SetFloatingPointPropertyValue(DataPtr, Value); });
}


void SNumericPropertyValue::OnValueCommitted_uint64(uint64 Value, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		OnValueChanged_uint64(Value);
	}
}
void SNumericPropertyValue::OnValueCommitted_int64(int64 Value, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		OnValueChanged_int64(Value);
	}
}
void SNumericPropertyValue::OnValueCommitted_double(double Value, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter || CommitInfo == ETextCommit::OnUserMovedFocus)
	{
		OnValueChanged_double(Value);
	}
}

} //namespace

#undef LOCTEXT_NAMESPACE
