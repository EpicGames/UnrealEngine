// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PropertyViewer/SObjectPropertyValue.h"

#include "Framework/PropertyViewer/INotifyHook.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#if WITH_EDITOR
#include "PropertyCustomizationHelpers.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#endif

namespace UE::PropertyViewer
{

	TSharedPtr<SWidget> SObjectPropertyValue::CreateInstance(const FPropertyValueFactory::FGenerateArgs Args)
	{
		return SNew(SObjectPropertyValue)
			.Path(Args.Path)
			.NotifyHook(Args.NotifyHook)
			.IsEnabled(Args.bCanEditValue);
	}


	void SObjectPropertyValue::Construct(const FArguments& InArgs)
	{
		Path = InArgs._Path;
		NotifyHook = InArgs._NotifyHook;

#if WITH_EDITOR
		if (const FProperty* Property = Path.GetLastProperty())
		{
			if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
			{
				if (Property->ArrayDim == 1)
				{
					ChildSlot
					[
						SNew(SObjectPropertyEntryBox)
						.ObjectPath(this, &SObjectPropertyValue::GetObjectPath)
						.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
						.AllowedClass(ObjectProperty->PropertyClass)
						.AllowCreate(false)
						.DisplayCompactSize(true)
						.DisplayThumbnail(false)
						.OnObjectChanged(this, &SObjectPropertyValue::OnObjectSelect)
					];
				}
			}
		}
#endif
	}

#if WITH_EDITOR
	FString SObjectPropertyValue::GetObjectPath() const
	{
		if (const void* Container = Path.GetContainerPtr())
		{
			if (const FProperty* Property = Path.GetLastProperty())
			{
				if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
				{
					FString ObjPath;
					ObjectProperty->ExportText_InContainer(0, ObjPath, Container, nullptr, nullptr, PPF_ExportsNotFullyQualified);
					return ObjPath;
				}
			}
		}

		return "";
	}

	void SObjectPropertyValue::OnObjectSelect(const FAssetData& InAsset)
	{
		if (void* Container = Path.GetContainerPtr())
		{
			if (const FProperty* Property = Path.GetLastProperty())
			{
				if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
				{
					if (NotifyHook)
					{
						NotifyHook->OnPreValueChange(Path);
					}

					ObjectProperty->SetObjectPropertyValue_InContainer(Container, InAsset.GetAsset());

					if (NotifyHook)
					{
						NotifyHook->OnPostValueChange(Path);
					}
				}
			}
		}
	}
#endif

} //namespace
