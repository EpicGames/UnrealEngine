// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PropertyViewer/SPropertyViewer.h"
#include "Widgets/PropertyViewer/SPropertyViewerImpl.h"

#define LOCTEXT_NAMESPACE "SPropertyViewer"

namespace UE::PropertyViewer
{

void SPropertyViewer::ConstructInternal(const FArguments& InArgs)
{
	ChildSlot
	[
		Implementation->Construct(InArgs)
	];
}

void SPropertyViewer::Construct(const FArguments& InArgs)
{
	Implementation = MakeShared<Private::FPropertyViewerImpl>(InArgs);
	ConstructInternal(InArgs);
}
void SPropertyViewer::Construct(const FArguments& InArgs, const UScriptStruct* Struct)
{
	Implementation = MakeShared<Private::FPropertyViewerImpl>(InArgs);
	Implementation->AddContainer(MakeContainerIdentifier(), TOptional<FText>(), Struct);
	ConstructInternal(InArgs);
}
void SPropertyViewer::Construct(const FArguments& InArgs, const UScriptStruct* Struct, void* InData)
{
	Implementation = MakeShared<Private::FPropertyViewerImpl>(InArgs);
	Implementation->AddContainerInstance(MakeContainerIdentifier(), TOptional<FText>(), Struct, InData);
	ConstructInternal(InArgs);
}
void SPropertyViewer::Construct(const FArguments& InArgs, const UClass* Class)
{
	Implementation = MakeShared<Private::FPropertyViewerImpl>(InArgs);
	Implementation->AddContainer(MakeContainerIdentifier(), TOptional<FText>(), Class);
	ConstructInternal(InArgs);
}
void SPropertyViewer::Construct(const FArguments& InArgs, UObject* ObjectInstance)
{
	Implementation = MakeShared<Private::FPropertyViewerImpl>(InArgs);
	Implementation->AddContainerInstance(MakeContainerIdentifier(), TOptional<FText>(), ObjectInstance);
	ConstructInternal(InArgs);
}
void SPropertyViewer::Construct(const FArguments& InArgs, const UFunction* Function)
{
	Implementation = MakeShared<Private::FPropertyViewerImpl>(InArgs);
	Implementation->AddContainer(MakeContainerIdentifier(), TOptional<FText>(), Function);
	ConstructInternal(InArgs);
}


void SPropertyViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	Implementation->Tick();
}


SPropertyViewer::FHandle SPropertyViewer::AddContainer(const UScriptStruct* Struct, TOptional<FText> DisplayName)
{
	SPropertyViewer::FHandle Result = MakeContainerIdentifier();
	Implementation->AddContainer(Result, DisplayName, Struct);
	return Result;
}
SPropertyViewer::FHandle SPropertyViewer::AddContainer(const UClass* Class, TOptional<FText> DisplayName)
{
	SPropertyViewer::FHandle Result = MakeContainerIdentifier();
	Implementation->AddContainer(Result, DisplayName, Class);
	return Result;
}
SPropertyViewer::FHandle SPropertyViewer::AddContainer(const UFunction* Function, TOptional<FText> DisplayName)
{
	SPropertyViewer::FHandle Result = MakeContainerIdentifier();
	Implementation->AddContainer(Result, DisplayName, Function);
	return Result;
}


SPropertyViewer::FHandle SPropertyViewer::AddInstance(const UScriptStruct* Struct, void* InData, TOptional<FText> DisplayName)
{
	check(InData);
	SPropertyViewer::FHandle Result = MakeContainerIdentifier();
	Implementation->AddContainerInstance(Result, DisplayName, Struct, InData);
	return Result;
}
SPropertyViewer::FHandle SPropertyViewer::AddInstance(UObject* ObjectInstance, TOptional<FText> DisplayName)
{
	SPropertyViewer::FHandle Result = MakeContainerIdentifier();
	Implementation->AddContainerInstance(Result, DisplayName, ObjectInstance);
	return Result;
}


void SPropertyViewer::Remove(FHandle Identifier)
{
	Implementation->Remove(Identifier);
}


void SPropertyViewer::RemoveAll()
{
	Implementation->RemoveAll();
}


TArray<SPropertyViewer::FSelectedItem> SPropertyViewer::GetSelectedItems() const
{
	return Implementation->GetSelectedItems();
}


void SPropertyViewer::SetRawFilterText(const FText& InFilterText)
{
	Implementation->SetRawFilterText(InFilterText);
}

void SPropertyViewer::SetSelection(FHandle Container, TArrayView<const FFieldVariant> FieldPath)
{
	Implementation->SetSelection(Container, FieldPath);
}

SPropertyViewer::FHandle SPropertyViewer::MakeContainerIdentifier()
{
	static int32 IdentifierGenerator = 0;
	SPropertyViewer::FHandle Result;
	++IdentifierGenerator;
	Result.Id = IdentifierGenerator;
	return Result;
}

} //namespace

#undef LOCTEXT_NAMESPACE
