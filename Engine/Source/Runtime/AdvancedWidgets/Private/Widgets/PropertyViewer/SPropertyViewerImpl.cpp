// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PropertyViewer/SPropertyViewerImpl.h"

#include "AdvancedWidgetsModule.h"
#include "Misc/IFilter.h"
#include "Misc/TextFilter.h"
#include "Framework/PropertyViewer/IFieldExpander.h"
#include "Framework/PropertyViewer/IFieldIterator.h"
#include "Framework/PropertyViewer/INotifyHook.h"
#include "Framework/PropertyViewer/PropertyPath.h"
#include "Framework/PropertyViewer/PropertyValueFactory.h"
#include "Framework/Views/TreeFilterHandler.h"

#include "Styling/SlateIconFinder.h"

#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/PropertyViewer/SFieldName.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

#include "UObject/UnrealType.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#endif //WITH_EDITOR

#define LOCTEXT_NAMESPACE "SPropertyViewerImpl"

namespace UE::PropertyViewer::Private
{

/** 
 * Column name 
 */
static FName ColumnName_Field = "Field";
static FName ColumnName_PropertyValue = "FieldValue";
static FName ColumnName_FieldPostWidget = "FieldPostWidget";


/**
 * FContainer
 */
FContainer::FContainer(SPropertyViewer::FHandle InIdentifier, TOptional<FText> InDisplayName, const UStruct* ClassToDisplay)
	: Identifier(InIdentifier)
	, Container(ClassToDisplay)
	, DisplayName(InDisplayName)
{
}


FContainer::FContainer(SPropertyViewer::FHandle InIdentifier, TOptional<FText> InDisplayName, UObject* InstanceToDisplay)
	: Identifier(InIdentifier)
	, Container(InstanceToDisplay->GetClass())
	, ObjectInstance(InstanceToDisplay)
	, DisplayName(InDisplayName)
	, bIsObject(true)
{
}


FContainer::FContainer(SPropertyViewer::FHandle InIdentifier, TOptional<FText> InDisplayName, const UScriptStruct* Struct, void* Data)
	: Identifier(InIdentifier)
	, Container(Struct)
	, StructInstance(Data)
	, DisplayName(InDisplayName)
{
}


bool InternalIsValid(const UStruct* Container)
{
	if (const UClass* Class = Cast<UClass>(Container))
	{
		return !Class->HasAnyClassFlags(EClassFlags::CLASS_NewerVersionExists);
	}
	else if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Container))
	{
		return (ScriptStruct->StructFlags & (EStructFlags::STRUCT_Trashed)) == 0;
	}
	return false;
}

bool FContainer::IsValid() const
{
	const UStruct * ContainerPtr = Container.Get();
	if (const UClass* Class = Cast<UClass>(ContainerPtr))
	{
		return !Class->HasAnyClassFlags(EClassFlags::CLASS_NewerVersionExists)
			&& (!bIsObject || (ObjectInstance.Get() && ObjectInstance.Get()->GetClass() == Class));
	}
	else if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(ContainerPtr))
	{
		return (ScriptStruct->StructFlags & (EStructFlags::STRUCT_Trashed)) == 0;
	}
	return false;
}


/**
 * FTreeNode
 */
TSharedRef<FTreeNode> FTreeNode::MakeContainer(const TSharedPtr<FContainer>& InContainer, TOptional<FText> InDisplayName)
{
	TSharedRef<FTreeNode> Result = MakeShared<FTreeNode>();
	Result->Container = InContainer;
	Result->OverrideDisplayName = InDisplayName;
	return Result;
}


TSharedRef<FTreeNode> FTreeNode::MakeField(TSharedPtr<FTreeNode> InParent, const FProperty* Property, TOptional<FText> InDisplayName)
{
	check(Property);
	TSharedRef<FTreeNode> Result = MakeShared<FTreeNode>();
	Result->FieldOwner = Property->GetOwnerStruct();
	Result->PropertyName = Property->GetFName();
	Result->bIsStructProperty = CastField<const FStructProperty>(Property) != nullptr;
	Result->bIsObjectProperty = CastField<const FObjectPropertyBase>(Property) != nullptr;
	Result->OverrideDisplayName = InDisplayName;
	Result->ParentNode = InParent;
	InParent->ChildNodes.Add(Result);
	return Result;
}


TSharedRef<FTreeNode> FTreeNode::MakeField(TSharedPtr<FTreeNode> InParent, const UFunction* Function, TOptional<FText> InDisplayName)
{
	check(Function);
	TSharedRef<FTreeNode> Result = MakeShared<FTreeNode>();
	Result->FieldOwner = Function->GetOwnerClass();
	Result->FunctionName = Function->GetFName();
	Result->OverrideDisplayName = InDisplayName;
	Result->ParentNode = InParent;
	InParent->ChildNodes.Add(Result);
	return Result;
}

FFieldVariant FTreeNode::GetField() const
{
	if (const UStruct* FieldOwnerPtr = FieldOwner.Get())
	{
		if (!PropertyName.IsNone())
		{
			return FFieldVariant(FieldOwnerPtr->FindPropertyByName(PropertyName));
		}
		else if (!FunctionName.IsNone())
		{
			return FFieldVariant(CastChecked<const UClass>(FieldOwnerPtr)->FindFunctionByName(FunctionName));
		}
	}
	return FFieldVariant();
}


FPropertyPath FTreeNode::GetPropertyPath() const
{
	const FTreeNode* CurrentNode = this;
	FPropertyPath::FPropertyArray Properties;
	while (CurrentNode)
	{
		const UStruct* CurrentFieldOwner = CurrentNode->FieldOwner.Get();
		if (CurrentFieldOwner && !CurrentNode->PropertyName.IsNone())
		{
			FProperty* Property = CurrentFieldOwner->FindPropertyByName(CurrentNode->PropertyName);
			if (Property)
			{
				Properties.Insert(Property, 0);
			}
			else
			{
				return FPropertyPath();
			}
		}

		TSharedPtr<FContainer> ContainerPin = CurrentNode->Container.Pin();
		if (ContainerPin)
		{
			if (ContainerPin->IsObjectInstance())
			{
				if (UObject* ObjectInstance = ContainerPin->GetObjectInstance())
				{
					return FPropertyPath(ObjectInstance, MoveTemp(Properties));
				}
			}
			else if (ContainerPin->IsScriptStructInstance())
			{
				if (const UStruct* ContainerStruct = ContainerPin->GetStruct())
				{
					return FPropertyPath(CastChecked<const UScriptStruct>(ContainerStruct), ContainerPin->GetScriptStructInstance(), MoveTemp(Properties));
				}
			}
			else if (const UClass* ContainerClass = Cast<const UClass>(ContainerPin->GetStruct()))
			{
				return FPropertyPath(ContainerClass->GetDefaultObject(), MoveTemp(Properties));
			}
			return FPropertyPath();
		}

		if (TSharedPtr<FTreeNode> ParentNodePin = CurrentNode->ParentNode.Pin())
		{
			// The property path are temporary for function
			if (ParentNodePin->GetField().Get<UFunction>())
			{
				return FPropertyPath();
			}
			CurrentNode = ParentNodePin.Get();
		}
		else if (ContainerPin == nullptr)
		{
			return FPropertyPath();
		}
	}
	return FPropertyPath();
}

TArray<FFieldVariant> FTreeNode::GetFieldPath() const
{
	TArray<FFieldVariant> Fields;

	const FTreeNode* CurrentNode = this;
	while (CurrentNode)
	{ 
		if (FFieldVariant Field = CurrentNode->GetField())
		{
			Fields.Insert(Field, 0);
		}
		
		CurrentNode = CurrentNode->ParentNode.Pin().Get();
	}
	return Fields;
}

TSharedPtr<FContainer> FTreeNode::GetOwnerContainer() const
{
	TSharedPtr<FContainer> Result;
	const FTreeNode* CurrentNode = this;
	while(CurrentNode)
	{
		if (TSharedPtr<FContainer> ContainerPin = CurrentNode->Container.Pin())
		{
			return ContainerPin;
		}
		if (TSharedPtr<FTreeNode> ParentNodePin = CurrentNode->ParentNode.Pin())
		{
			CurrentNode = ParentNodePin.Get();
		}
		else
		{
			ensureMsgf(false, TEXT("The tree is not owned by a container"));
			break;
		}
	}
	return TSharedPtr<FContainer>();
}


void FTreeNode::GetFilterStrings(TArray<FString>& OutStrings) const
{
	const UStruct* CurrentFieldOwner = FieldOwner.Get();
	if (CurrentFieldOwner && !PropertyName.IsNone())
	{
		FProperty* PropertyPtr = CurrentFieldOwner->FindPropertyByName(PropertyName);
		if (PropertyPtr)
		{
			OutStrings.Add(PropertyPtr->GetName());
#if WITH_EDITORONLY_DATA
			OutStrings.Add(PropertyPtr->GetDisplayNameText().ToString());
#endif
		}
	}
	if (CurrentFieldOwner && !FunctionName.IsNone())
	{
		const UFunction* FunctionPtr = CastChecked<const UClass>(CurrentFieldOwner)->FindFunctionByName(FunctionName);
		if (FunctionPtr)
		{
			OutStrings.Add(FunctionPtr->GetName());
#if WITH_EDITORONLY_DATA
			OutStrings.Add(FunctionPtr->GetDisplayNameText().ToString());
#endif
		}
	}
	if (const TSharedPtr<FContainer> ContainerPtr = Container.Pin())
	{
		if (const UStruct* StructPtr = ContainerPtr->GetStruct())
		{
			OutStrings.Add(StructPtr->GetName());
#if WITH_EDITORONLY_DATA
			OutStrings.Add(StructPtr->GetDisplayNameText().ToString());
#endif
		}
	}
	if (GetOverrideDisplayName().IsSet())
	{
		OutStrings.Add(GetOverrideDisplayName().GetValue().ToString());
	}
}


TArray<FTreeNode::FNodeReference> FTreeNode::BuildChildNodes(IFieldIterator& FieldIterator, IFieldExpander& FieldExpander, bool bSortChildNode)
{
	TArray<FNodeReference> TickReferences;
	BuildChildNodesRecursive(FieldIterator, FieldExpander, bSortChildNode, 2, TickReferences);
	return TickReferences;
}


void FTreeNode::BuildChildNodesRecursive(IFieldIterator& FieldIterator, IFieldExpander& FieldExpander, bool bSortChildNode, int32 RecursiveCount, TArray<FNodeReference>& OutTickReference)
{
	if (RecursiveCount <= 0)
	{
		return;
	}
	--RecursiveCount;

	ChildNodes.Reset();

	const UStruct* ChildStructType = nullptr;
	const UStruct* CurrentFieldOwner = nullptr;
	FName CurrentFieldName = NAME_None;
	if (bIsStructProperty || bIsObjectProperty)
	{
		check(!PropertyName.IsNone());
		CurrentFieldName = PropertyName;
		CurrentFieldOwner = FieldOwner.Get();
		FProperty* PropertyPtr = CurrentFieldOwner ? CurrentFieldOwner->FindPropertyByName(PropertyName) : nullptr;
		if (const FStructProperty* StructProperty = CastField<const FStructProperty>(PropertyPtr))
		{
			if (FieldExpander.CanExpandScriptStruct(StructProperty))
			{
				ChildStructType = StructProperty->Struct;
			}
		}
		else if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(PropertyPtr))
		{
			UObject* Instance = nullptr;
			if (TSharedPtr<FContainer> OwnerContainer = GetOwnerContainer())
			{
				if (OwnerContainer->IsInstance())
				{
					const FPropertyPath PropertyPath = GetPropertyPath();
					if (const void* ContainerPtr = PropertyPath.GetContainerPtr())
					{
						Instance = ObjectProperty->GetObjectPropertyValue_InContainer(ContainerPtr);
					}
				}
			}

			TOptional<const UClass*> ClassToExpand = FieldExpander.CanExpandObject(ObjectProperty, Instance);
			if (ClassToExpand.IsSet())
			{
				if (Instance)
				{
					OutTickReference.Emplace(Instance, AsShared());
				}
				ChildStructType = ClassToExpand.GetValue();
			}
		}
	}
	else if (!FunctionName.IsNone())
	{
		CurrentFieldOwner = FieldOwner.Get();
		CurrentFieldName = FunctionName;
		UFunction* FunctionPtr = CurrentFieldOwner ? CastChecked<const UClass>(CurrentFieldOwner)->FindFunctionByName(FunctionName) : nullptr;
		if (FunctionPtr)
		{
			TOptional<const UStruct*> StructToExpand = FieldExpander.GetExpandedFunction(FunctionPtr);
			if (StructToExpand.IsSet())
			{
				ChildStructType = StructToExpand.GetValue();
			}
		}
	}
	else if (const TSharedPtr<FContainer> ContainerPin = Container.Pin())
	{
		ChildStructType = ContainerPin->GetStruct();
		CurrentFieldName = FName(ContainerPin->GetDisplayName().Get(FText::GetEmpty()).ToString());
	}

	if (ChildStructType)
	{
		for (const FFieldVariant& FieldIt : FieldIterator.GetFields(ChildStructType, CurrentFieldName, CurrentFieldOwner))
		{
			if (const FProperty* PropertyIt = FieldIt.Get<FProperty>())
			{
				TSharedPtr<FTreeNode> Node = MakeField(AsShared(), PropertyIt, TOptional<FText>());
				Node->BuildChildNodesRecursive(FieldIterator, FieldExpander, bSortChildNode, RecursiveCount, OutTickReference);
			}
			if (const UFunction* FunctionIt = FieldIt.Get<UFunction>())
			{
				TSharedPtr<FTreeNode> Node = MakeField(AsShared(), FunctionIt, TOptional<FText>());
				Node->BuildChildNodesRecursive(FieldIterator, FieldExpander, bSortChildNode, RecursiveCount, OutTickReference);
			}
		}

		if (bSortChildNode)
		{
			ChildNodes.Sort(Sort);
		}
	}

	bChildGenerated = true;
}


bool FTreeNode::Sort(const TSharedPtr<FTreeNode>& NodeA, const TSharedPtr<FTreeNode>& NodeB)
{
	bool bIsContainerA = NodeA->IsContainer();
	bool bIsContainerB = NodeB->IsContainer();
	bool bIsObjectPropertyA = NodeA->bIsObjectProperty;
	bool bIsObjectPropertyB = NodeB->bIsObjectProperty;
	bool bIsFunctionA = !NodeA->FunctionName.IsNone();
	bool bIsFunctionB = !NodeB->FunctionName.IsNone();
	const FName NodeStrA = bIsContainerA ? NodeA->GetContainer()->GetStruct()->GetFName() : NodeA->GetField().GetFName();
	const FName NodeStrB = bIsContainerB ? NodeB->GetContainer()->GetStruct()->GetFName() : NodeB->GetField().GetFName();

	if (bIsFunctionA && bIsFunctionB)
	{
		return NodeStrA.LexicalLess(NodeStrB);
	}
	if (bIsObjectPropertyA && bIsObjectPropertyB)
	{
		return NodeStrA.LexicalLess(NodeStrB);
	}

	if (bIsFunctionA)
	{
		return true;
	}
	if (bIsFunctionB)
	{
		return false;
	}
	if (bIsObjectPropertyA)
	{
		return true;
	}
	if (bIsObjectPropertyB)
	{
		return false;
	}

	return NodeStrA.LexicalLess(NodeStrB);
};


/**
 * FPropertyViewerImpl
 */
FPropertyViewerImpl::FPropertyViewerImpl(const SPropertyViewer::FArguments& InArgs)
{
	FieldIterator = InArgs._FieldIterator;
	FieldExpander = InArgs._FieldExpander;
	if (FieldIterator == nullptr)
	{
		FieldIterator = new FFieldIterator_BlueprintVisible();
		bOwnFieldIterator = true;
	}
	if (FieldExpander == nullptr)
	{
		FFieldExpander_Default* DefaultFieldExpander = new FFieldExpander_Default();
		DefaultFieldExpander->SetExpandObject(FFieldExpander_Default::EObjectExpandFlag::None);
		DefaultFieldExpander->SetExpandScriptStruct(true);
		FieldExpander = DefaultFieldExpander;
		bOwnFieldExpander = true;
	}

	NotifyHook = InArgs._NotifyHook;
	OnGetPreSlot = InArgs._OnGetPreSlot;
	OnGetPostSlot = InArgs._OnGetPostSlot;
	OnContextMenuOpening = InArgs._OnContextMenuOpening;
	OnSelectionChanged = InArgs._OnSelectionChanged;
	OnDoubleClicked = InArgs._OnDoubleClicked;
	OnDragDetected = InArgs._OnDragDetected;
	OnGenerateContainer = InArgs._OnGenerateContainer;
	PropertyVisibility = InArgs._PropertyVisibility;
	bSanitizeName = InArgs._bSanitizeName;
	bShowFieldIcon = InArgs._bShowFieldIcon;
	bSortChildNode = InArgs._bSortChildNode;

#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->OnBlueprintCompiled().AddRaw(this, &FPropertyViewerImpl::HandleBlueprintCompiled);
		FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FPropertyViewerImpl::HandleReplaceViewedObjects);
	}
#endif
}


FPropertyViewerImpl::~FPropertyViewerImpl()
{
	if (bOwnFieldIterator)
	{
		delete FieldIterator;
	}
	if (bOwnFieldExpander)
	{
		delete FieldExpander;
	}
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->OnBlueprintCompiled().RemoveAll(this);
		FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	}
#endif
}


TSharedRef<SWidget> FPropertyViewerImpl::Construct(const SPropertyViewer::FArguments& InArgs)
{
	SearchFilter = MakeShared<FTextFilter>(FTextFilter::FItemToStringArray::CreateSP(this, &FPropertyViewerImpl::HandleGetFilterStrings));

	FilterHandler = MakeShared<FTreeFilter>();
	FilterHandler->SetFilter(SearchFilter.Get());
	FilterHandler->SetRootItems(&TreeSource, &FilteredTreeSource);
	FilterHandler->SetGetChildrenDelegate(FTreeFilter::FOnGetChildren::CreateRaw(this, &FPropertyViewerImpl::HandleGetChildren));

	OverrideIconColorSettings = InArgs._OverrideIconColorSettings;

	TSharedPtr<SHorizontalBox> SearchBox;
	if (InArgs._bShowSearchBox || InArgs._SearchBoxPreSlot.Widget != SNullWidget::NullWidget || InArgs._SearchBoxPostSlot.Widget != SNullWidget::NullWidget)
	{
		SearchBox = SNew(SHorizontalBox);
		if (InArgs._SearchBoxPreSlot.Widget != SNullWidget::NullWidget)
		{
			SearchBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0, 0, 4, 0)
				[
					InArgs._SearchBoxPreSlot.Widget
				];
		}

		if (InArgs._bShowSearchBox)
		{
			SearchBox->AddSlot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[
					CreateSearch()
				];

			if (InArgs._bFocusSearchBox)
			{
				SearchBox->RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSPLambda(this, [this](double InCurrentTime, float InDeltaTime)
				{
					if (SearchBoxWidget.IsValid())
					{
						FWidgetPath WidgetToFocusPath;
						FSlateApplication::Get().GeneratePathToWidgetUnchecked(SearchBoxWidget.ToSharedRef(), WidgetToFocusPath);
						if(WidgetToFocusPath.IsValid())
						{
							FSlateApplication::Get().SetKeyboardFocus(WidgetToFocusPath, EFocusCause::SetDirectly);
							WidgetToFocusPath.GetWindow()->SetWidgetToFocusOnActivate(SearchBoxWidget);
						}
						return EActiveTimerReturnType::Stop;
					}

					return EActiveTimerReturnType::Continue;
				}));
			}
		}
		else
		{
			SearchBox->AddSlot()
				.FillWidth(1.f)
				[
					SNullWidget::NullWidget
				];
		}

		if (InArgs._SearchBoxPostSlot.Widget != SNullWidget::NullWidget)
		{
			SearchBox->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(4, 0, 0, 0)
				[
					InArgs._SearchBoxPostSlot.Widget
				];
		}
	}

	TSharedRef<SWidget> ConstructedTree = SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		.Padding(0)
		[
			CreateTree(OnGetPreSlot.IsBound(), PropertyVisibility != SPropertyViewer::EPropertyVisibility::Hidden, OnGetPostSlot.IsBound(), InArgs._SelectionMode)
		];

	if (SearchBox)
	{
		return SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(4)
			.AutoHeight()
			[
				SearchBox.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				ConstructedTree
			];
	}
	else
	{
		return ConstructedTree;
	}
}


void FPropertyViewerImpl::Tick()
{
	auto RemoveChild = [](const TSharedPtr<STreeView<TSharedPtr<FTreeNode>>>& InTreeWidget, const TSharedPtr<FTreeNode>& InNodePin)
		{
			if (TSharedPtr<FTreeNode> ParentNode = InNodePin->GetParentNode())
			{
				ParentNode->RemoveChild();
			}
			InTreeWidget->RequestListRefresh();
		};

	// If the object instance is not the same as the previous frame. Rebuild the tree.
	for (int32 Index = NodeReferences.Num() - 1; Index >= 0; --Index)
	{
		UObject* PreviousPtr = NodeReferences[Index].Previous.Get();
		TSharedPtr<FTreeNode> NodePin = NodeReferences[Index].Node.Pin();
		if (PreviousPtr && NodePin)
		{
			UObject* Instance = nullptr;
			if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(NodePin->GetField().Get<FProperty>()))
			{
				const FPropertyPath PropertyPath = NodePin->GetPropertyPath();
				if (const void* ContainerPtr = PropertyPath.GetContainerPtr())
				{
					Instance = ObjectProperty->GetObjectPropertyValue_InContainer(ContainerPtr);
				}
			}

			if (Instance != PreviousPtr)
			{
				RemoveChild(TreeWidget, NodePin);
				NodeReferences.RemoveAtSwap(Index);
			}
		}
		else
		{
			if (NodePin)
			{
				RemoveChild(TreeWidget, NodePin);
			}
			NodeReferences.RemoveAtSwap(Index);
		}
	}
}


void FPropertyViewerImpl::AddContainer(SPropertyViewer::FHandle Identifier, TOptional<FText> DisplayName, const UStruct* Struct)
{
	TSharedPtr<FContainer> NewContainer = MakeShared<FContainer>(Identifier, DisplayName, Struct);
	Containers.Add(NewContainer);

	AddContainerInternal(Identifier, NewContainer);
}


void FPropertyViewerImpl::AddContainerInstance(SPropertyViewer::FHandle Identifier, TOptional<FText> DisplayName, UObject* Object)
{
	TSharedPtr<FContainer> NewContainer = MakeShared<FContainer>(Identifier, DisplayName, Object);
	Containers.Add(NewContainer);

	AddContainerInternal(Identifier, NewContainer);
}


void FPropertyViewerImpl::AddContainerInstance(SPropertyViewer::FHandle Identifier, TOptional<FText> DisplayName, const UScriptStruct* Struct, void* Data)
{
	TSharedPtr<FContainer> NewContainer = MakeShared<FContainer>(Identifier, DisplayName, Struct, Data);
	Containers.Add(NewContainer);

	AddContainerInternal(Identifier, NewContainer);
}


void FPropertyViewerImpl::AddContainerInternal(SPropertyViewer::FHandle Identifier, TSharedPtr<FContainer>& NewContainer)
{
	TSharedPtr<FTreeNode> NewNode = FTreeNode::MakeContainer(NewContainer, NewContainer->GetDisplayName());
	NodeReferences.Append(NewNode->BuildChildNodes(*FieldIterator, *FieldExpander, bSortChildNode));
	TreeSource.Add(NewNode);

	if (TreeWidget)
	{
		TreeWidget->SetItemExpansion(NewNode, true);
	}
	if (FilterHandler)
	{
		FilterHandler->RefreshAndFilterTree();
	}
}


void FPropertyViewerImpl::Remove(SPropertyViewer::FHandle Identifier)
{
	bool bRemoved = false;
	for (int32 Index = 0; Index < TreeSource.Num(); ++Index)
	{
		if (TSharedPtr<FContainer> Container = TreeSource[Index]->GetContainer())
		{
			if (Container->GetIdentifier() == Identifier)
			{
				TreeSource.RemoveAt(Index);
				bRemoved = true;
				break;
			}
		}
	}

	for (int32 Index = 0; Index < Containers.Num(); ++Index)
	{
		if (Containers[Index]->GetIdentifier() == Identifier)
		{
			Containers.RemoveAt(Index);
			break;
		}
	}

	if (bRemoved)
	{
		if (FilterHandler)
		{
			FilterHandler->RefreshAndFilterTree();
		}
		else
		{
			TreeWidget->RequestTreeRefresh();
		}
	}
}


void FPropertyViewerImpl::RemoveAll()
{
	bool bRemoved = TreeSource.Num() > 0 || Containers.Num() > 0;
	TreeSource.Reset();
	Containers.Reset();

	if (bRemoved)
	{
		if (FilterHandler)
		{
			FilterHandler->RefreshAndFilterTree();
		}
		else
		{
			TreeWidget->RequestTreeRefresh();
		}
	}
}


TSharedRef<SWidget> FPropertyViewerImpl::CreateSearch()
{
	return SAssignNew(SearchBoxWidget, SSearchBox)
		.HintText(LOCTEXT("SearchHintText", "Search"))
		.OnTextChanged(this, &FPropertyViewerImpl::HandleSearchChanged);
}


TArray<SPropertyViewer::FSelectedItem> FPropertyViewerImpl::GetSelectedItems() const
{
	TArray<SPropertyViewer::FSelectedItem> Result;

	TArray<TSharedPtr<FTreeNode>> SelectedItem = TreeWidget->GetSelectedItems();
	for (TSharedPtr<FTreeNode> Node : SelectedItem)
	{
		if (TSharedPtr<FContainer> Container = Node->GetContainer())
		{
			int32 FoundIndex = Result.IndexOfByPredicate([Container](const SPropertyViewer::FSelectedItem& Other) { return Other.Handle == Container->GetIdentifier(); });
			if (FoundIndex == INDEX_NONE)
			{
				SPropertyViewer::FSelectedItem Item;
				Item.Handle = Container->GetIdentifier();
				FoundIndex = Result.Add(Item);
			}
			Result[FoundIndex].bIsContainerSelected = true;
		}
		else
		{
			if (TSharedPtr<FContainer> OwnerContainer = Node->GetOwnerContainer())
			{
				int32 FoundIndex = Result.IndexOfByPredicate([OwnerContainer](const SPropertyViewer::FSelectedItem& Other){ return Other.Handle == OwnerContainer->GetIdentifier(); });
				if (FoundIndex == INDEX_NONE)
				{
					SPropertyViewer::FSelectedItem Item;
					Item.Handle = OwnerContainer->GetIdentifier();
					FoundIndex = Result.Add(Item);
				}
				Result[FoundIndex].Fields.Add(Node->GetFieldPath());
			}
		}
	}
	return Result;
}


void FPropertyViewerImpl::SetRawFilterText(const FText& InFilterText)
{
	SetRawFilterTextInternal(InFilterText);
}


FText FPropertyViewerImpl::SetRawFilterTextInternal(const FText& InFilterText)
{
	const bool bNewFilteringEnabled = !InFilterText.IsEmpty();
	FilterHandler->SetIsEnabled(bNewFilteringEnabled);
	SearchFilter->SetRawFilterText(InFilterText);
	FilterHandler->RefreshAndFilterTree();

	for (const TSharedPtr<FTreeNode>& Node : FilteredTreeSource)
	{
		SetHighlightTextRecursive(Node, InFilterText);
	}

	return SearchFilter->GetFilterErrorText();
}

TSharedPtr<FTreeNode> FPropertyViewerImpl::FindExistingChild(const TSharedPtr<FTreeNode>& Node, TArrayView<const FFieldVariant> FieldPath) const
{
	if (FieldPath.Num() == 0)
	{
		return TSharedPtr<FTreeNode>();
	}

	for (const TSharedPtr<FTreeNode>& Child : Node->ChildNodes)
	{
		if (Child->GetField() == FieldPath[0])
		{
			if (FieldPath.Num() == 1)
			{
				return Child;
			}
			else
			{
				return FindExistingChild(Child, FieldPath.RightChop(1));
			}
		}
	}

	return TSharedPtr<FTreeNode>();
}

void FPropertyViewerImpl::SetSelection(SPropertyViewer::FHandle Identifier, TArrayView<const FFieldVariant> FieldPath)
{
	TreeWidget->ClearSelection();
	for (const TSharedPtr<FTreeNode>& Node : TreeSource)
	{
		if (TSharedPtr<FContainer> Container = Node->GetContainer())
		{
			if (Container->GetIdentifier() == Identifier)
			{
				if (TSharedPtr<FTreeNode> FoundNode = FindExistingChild(Node, FieldPath))
				{
					TreeWidget->SetItemSelection(FoundNode, true);
					break;
				}
			}
		}
	}
}


TSharedRef<SWidget> FPropertyViewerImpl::CreateTree(bool bHasPreWidget, bool bShowPropertyValue, bool bHasPostWidget, ESelectionMode::Type SelectionMode)
{
	TSharedPtr<SHeaderRow> HeaderRowWidget;
	if (bShowPropertyValue || bHasPostWidget)
	{
		bUseRows = true;

		HeaderRowWidget = SNew(SHeaderRow)
		.Visibility(EVisibility::Collapsed);

		HeaderRowWidget->AddColumn(
			SHeaderRow::FColumn::FArguments()
			.ColumnId(ColumnName_Field)
			.DefaultLabel(LOCTEXT("FieldName", "Field Name"))
			.FillWidth(0.75f)
		);

		if (bShowPropertyValue)
		{
			HeaderRowWidget->AddColumn(
				SHeaderRow::FColumn::FArguments()
				.ColumnId(ColumnName_PropertyValue)
				.MinSize(150.f)
				.DefaultLabel(LOCTEXT("PropertyValue", "Field Value"))
				.FillWidth(0.25f)
			);
		}
		if (bHasPostWidget)
		{
			HeaderRowWidget->AddColumn(
				SHeaderRow::FColumn::FArguments()
				.ColumnId(ColumnName_FieldPostWidget)
				.DefaultLabel(FText())
			);
		}
	}

	if (FilterHandler)
	{
		SAssignNew(TreeWidget, STreeView<TSharedPtr<FTreeNode>>)
			.TreeItemsSource(&FilteredTreeSource)
			.SelectionMode(SelectionMode)
			.OnGetChildren(FilterHandler.ToSharedRef(), &FTreeFilter::OnGetFilteredChildren)
			.OnGenerateRow(this, &FPropertyViewerImpl::HandleGenerateRow)
			.OnSelectionChanged(this, &FPropertyViewerImpl::HandleSelectionChanged)
			.OnContextMenuOpening(this, &FPropertyViewerImpl::HandleContextMenuOpening)
			.OnMouseButtonDoubleClick(this, &FPropertyViewerImpl::HandleDoubleClick)
			.HeaderRow(HeaderRowWidget);
		FilterHandler->SetTreeView(TreeWidget.Get());
	}
	else
	{
		SAssignNew(TreeWidget, STreeView<TSharedPtr<FTreeNode>>)
			.TreeItemsSource(&TreeSource)
			.SelectionMode(SelectionMode)
			.OnGetChildren(this, &FPropertyViewerImpl::HandleGetChildren)
			.OnGenerateRow(this, &FPropertyViewerImpl::HandleGenerateRow)
			.OnSelectionChanged(this, &FPropertyViewerImpl::HandleSelectionChanged)
			.OnContextMenuOpening(this, &FPropertyViewerImpl::HandleContextMenuOpening)
			.OnMouseButtonDoubleClick(this, &FPropertyViewerImpl::HandleDoubleClick)
			.HeaderRow(HeaderRowWidget);
	}

	return TreeWidget.ToSharedRef();
}


void FPropertyViewerImpl::SetHighlightTextRecursive(const TSharedPtr<FTreeNode>& OwnerNode, const FText& HighlightText)
{
	if (TSharedPtr<SFieldName> PropertyNameWidget = OwnerNode->PropertyWidget.Pin())
	{
		PropertyNameWidget->SetHighlightText(HighlightText);
	}

	if (OwnerNode->bChildGenerated)
	{
		for (const TSharedPtr<FTreeNode>& Node : OwnerNode->ChildNodes)
		{
			SetHighlightTextRecursive(Node, HighlightText);
		}
	}
}


void FPropertyViewerImpl::HandleGetFilterStrings(TSharedPtr<FTreeNode> Item, TArray<FString>& OutStrings)
{
	Item->GetFilterStrings(OutStrings);
}


TSharedRef<ITableRow> FPropertyViewerImpl::HandleGenerateRow(TSharedPtr<FTreeNode> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	FText HighlightText = SearchFilter ? SearchFilter->GetRawFilterText() : FText::GetEmpty();

	TSharedPtr<SWidget> PreWidget;
	if (OnGetPreSlot.IsBound())
	{
		TSharedPtr<FContainer> OwnerContainer = Item->GetOwnerContainer();
		PreWidget = OnGetPreSlot.Execute(OwnerContainer.IsValid() ? OwnerContainer->GetIdentifier() : SPropertyViewer::FHandle(), Item->GetFieldPath());
	}

	TSharedPtr<SWidget> ItemWidget;
	if (TSharedPtr<FContainer> ContainerPin = Item->GetContainer())
	{
		if (OnGenerateContainer.IsBound())
		{
			ItemWidget = OnGenerateContainer.Execute(ContainerPin->GetIdentifier(), Item->GetOverrideDisplayName());
		}
		else if (ContainerPin->IsValid())
		{
			if (const UClass* Class = Cast<const UClass>(ContainerPin->GetStruct()))
			{
				ItemWidget = SNew(SFieldName, Class)
					.bShowIcon(true)
					.bSanitizeName(bSanitizeName)
					.OverrideDisplayName(Item->GetOverrideDisplayName())
					.OverrideIconColorSettings(OverrideIconColorSettings);
			}
			if (const UScriptStruct* Struct = Cast<const UScriptStruct>(ContainerPin->GetStruct()))
			{
				ItemWidget = SNew(SFieldName, Struct)
					.bShowIcon(true)
					.bSanitizeName(bSanitizeName)
					.OverrideDisplayName(Item->GetOverrideDisplayName())
					.OverrideIconColorSettings(OverrideIconColorSettings);
			}
		}
	}
	else if (TSharedPtr<FContainer> OwnerContainerPin = Item->GetOwnerContainer())
	{
		if (OwnerContainerPin->IsValid())
		{
			if (FFieldVariant FieldVariant = Item->GetField())
			{
				if (FProperty* Property = FieldVariant.Get<FProperty>())
				{
					TSharedPtr<SFieldName> FieldName = SNew(SFieldName, Property)
						.bShowIcon(bShowFieldIcon)
						.bSanitizeName(bSanitizeName)
						.OverrideDisplayName(Item->GetOverrideDisplayName())
						.OverrideIconColorSettings(OverrideIconColorSettings)
						.HighlightText(HighlightText);
					Item->PropertyWidget = FieldName;
					ItemWidget = FieldName;
				}
				else if (UFunction* Function = FieldVariant.Get<UFunction>())
				{
					TSharedPtr<SFieldName> FieldName = SNew(SFieldName, Function)
						.bShowIcon(bShowFieldIcon)
						.bSanitizeName(bSanitizeName)
						.OverrideDisplayName(Item->GetOverrideDisplayName())
						.OverrideIconColorSettings(OverrideIconColorSettings)
						.HighlightText(HighlightText);
					Item->PropertyWidget = FieldName;
					ItemWidget = FieldName;
				}
			}
		}
	}

	struct SMultiRowType : public SMultiColumnTableRow<TSharedPtr<FTreeNode>>
	{
		void Construct(const FArguments& Args, const TSharedRef<FPropertyViewerImpl> PropertyViewer, const TSharedRef<STableViewBase>& OwnerTableView, TSharedRef<FTreeNode> InItem, TSharedRef<SWidget> InFieldWidget)
		{
			PropertyViewOwner = PropertyViewer;
			Item = InItem;
			FieldWidget = InFieldWidget;
			SMultiColumnTableRow<TSharedPtr<FTreeNode>>::Construct(Args, OwnerTableView);
		}

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == ColumnName_Field)
			{
				TSharedRef<SHorizontalBox> Stack = SNew(SHorizontalBox);

				TSharedPtr<FPropertyViewerImpl> PropertyViewOwnerPin = PropertyViewOwner.Pin();
				TSharedPtr<SWidget> PreWidget;
				if (PropertyViewOwnerPin && PropertyViewOwnerPin->OnGetPreSlot.IsBound())
				{
					if (TSharedPtr<FTreeNode> ItemPin = Item.Pin())
					{						
						TSharedPtr<FContainer> OwnerContainer = ItemPin->GetOwnerContainer();
						PreWidget = PropertyViewOwnerPin->OnGetPreSlot.Execute(OwnerContainer.IsValid() ? OwnerContainer->GetIdentifier() : SPropertyViewer::FHandle(), ItemPin->GetFieldPath());
					}
				}

				Stack->AddSlot()
				.AutoWidth()
				[
					SNew(SExpanderArrow, SharedThis(this))
					.IndentAmount(16)
					.ShouldDrawWires(true)
				];

				if (PreWidget)
				{
					Stack->AddSlot()
					.AutoWidth()
					[
						PreWidget.ToSharedRef()
					];
				}

				Stack->AddSlot()
				.AutoWidth()
				.Padding(2.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					FieldWidget.ToSharedRef()
				];

				return Stack;
			}

			if (ColumnName == ColumnName_PropertyValue)
			{
				TSharedPtr<FTreeNode> ItemPin = Item.Pin();
				TSharedPtr<FPropertyViewerImpl> PropertyViewOwnerPin = PropertyViewOwner.Pin();
				if (ItemPin && PropertyViewOwnerPin)
				{
					if (ItemPin->IsField())
					{
						FFieldVariant Field = ItemPin->GetField();

						bool bCanEditContainer = false;
						if (const TSharedPtr<FContainer> OwnerContainer = ItemPin->GetOwnerContainer())
						{
							bCanEditContainer = OwnerContainer->CanEdit();
						}

						FPropertyValueFactory::FGenerateArgs Args;
						Args.Path = ItemPin->GetPropertyPath();
						Args.Field = &Field;
						Args.NotifyHook = PropertyViewOwnerPin->NotifyHook;
						Args.bCanEditValue = bCanEditContainer
							&& PropertyViewOwnerPin->PropertyVisibility == SPropertyViewer::EPropertyVisibility::Editable
							&& Args.Path.GetLastProperty() != nullptr
							&& !Args.Path.GetLastProperty()->HasAllPropertyFlags(CPF_BlueprintReadOnly);

						FAdvancedWidgetsModule& Module = FAdvancedWidgetsModule::GetModule();
						TSharedPtr<SWidget> ValueWidget = Module.GetPropertyValueFactory().Generate(Args);
						if (!ValueWidget)
						{
							ValueWidget = Module.GetPropertyValueFactory().GenerateDefault(Args);
						}

						if (ValueWidget)
						{
							return SNew(SBox)
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								[
									ValueWidget.ToSharedRef()
								];
						}
					}
				}
			}

			if (ColumnName == ColumnName_FieldPostWidget)
			{
				TSharedPtr<FTreeNode> ItemPin = Item.Pin();
				TSharedPtr<FPropertyViewerImpl> PropertyViewOwnerPin = PropertyViewOwner.Pin();
				if (ItemPin && PropertyViewOwnerPin)
				{
					TSharedPtr<FContainer> OwnerContainer = ItemPin->GetOwnerContainer();
					TSharedPtr<SWidget> PostWidget = PropertyViewOwnerPin->OnGetPostSlot.Execute(OwnerContainer.IsValid() ? OwnerContainer->GetIdentifier() : SPropertyViewer::FHandle(), ItemPin->GetFieldPath());
					if (PostWidget)
					{
						return PostWidget.ToSharedRef();
					}
				}
			}

			return SNullWidget::NullWidget;
		}

	private:
		TWeakPtr<FTreeNode> Item;
		TSharedPtr<SWidget> FieldWidget;
		TWeakPtr<FPropertyViewerImpl> PropertyViewOwner;
	};


	TSharedRef<SWidget> FieldWidget = ItemWidget ? ItemWidget.ToSharedRef() : SNullWidget::NullWidget;
	if (bUseRows)
	{
		return SNew(SMultiRowType, AsShared(), OwnerTable, Item.ToSharedRef(), FieldWidget)
			.OnDragDetected(this, &FPropertyViewerImpl::HandleDragDetected, Item)
			.Padding(0.0f);
	}

	using SSimpleRowType = STableRow<TSharedPtr<FTreeNode>>;

	if (PreWidget)
	{
		return SNew(SSimpleRowType, OwnerTable)
			.Padding(0.0f)
			.Content()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					PreWidget.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					FieldWidget
				]
			];
	}

	return SNew(SSimpleRowType, OwnerTable)
		.Padding(0.0f)
		.Content()
		[
			FieldWidget
		];
}


void FPropertyViewerImpl::HandleGetChildren(TSharedPtr<FTreeNode> InParent, TArray<TSharedPtr<FTreeNode>>& OutChildren)
{
	if (!InParent->bChildGenerated)
	{			
		// Do not build when filtering (only search in what it's already been built)
		if (FilterHandler == nullptr || !FilterHandler->GetIsEnabled())
		{
			NodeReferences.Append(InParent->BuildChildNodes(*FieldIterator, *FieldExpander, bSortChildNode));
		}
	}
	OutChildren = InParent->ChildNodes;
}


TSharedPtr<SWidget> FPropertyViewerImpl::HandleContextMenuOpening()
{
	if (OnContextMenuOpening.IsBound())
	{
		TArray<TSharedPtr<FTreeNode>> Items = TreeWidget->GetSelectedItems();
		if (Items.Num() == 1 && Items[0].IsValid())
		{
			TSharedPtr<FContainer> OwnerContainer = Items[0]->GetOwnerContainer();
			return OnContextMenuOpening.Execute(OwnerContainer.IsValid() ? OwnerContainer->GetIdentifier() : SPropertyViewer::FHandle(), Items[0]->GetFieldPath());
		}
	}
	return TSharedPtr<SWidget>();
}


void FPropertyViewerImpl::HandleSelectionChanged(TSharedPtr<FTreeNode> Item, ESelectInfo::Type SelectionType)
{
	if (OnSelectionChanged.IsBound())
	{
		if (Item.IsValid())
		{
			TSharedPtr<FContainer> OwnerContainer = Item->GetOwnerContainer();
			OnSelectionChanged.Execute(OwnerContainer.IsValid() ? OwnerContainer->GetIdentifier() : SPropertyViewer::FHandle(), Item->GetFieldPath(), SelectionType);
		}
		else
		{
			OnSelectionChanged.Execute(SPropertyViewer::FHandle(), TArray<FFieldVariant>(), SelectionType);
		}
	}
}

void FPropertyViewerImpl::HandleDoubleClick(TSharedPtr<FTreeNode> Item)
{
	if (OnDoubleClicked.IsBound())
	{
		if (Item.IsValid())
		{
			TSharedPtr<FContainer> OwnerContainer = Item->GetOwnerContainer();
			OnDoubleClicked.Execute(OwnerContainer.IsValid() ? OwnerContainer->GetIdentifier() : SPropertyViewer::FHandle(), Item->GetFieldPath());
		}
		else
		{
			OnDoubleClicked.Execute(SPropertyViewer::FHandle(), TArray<FFieldVariant>());
		}
	}
}


FReply FPropertyViewerImpl::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, const TSharedPtr<FTreeNode> Item)
{
	if (OnDragDetected.IsBound())
	{
		if (Item.IsValid())
		{
			TSharedPtr<FContainer> OwnerContainer = Item->GetOwnerContainer();
			return OnDragDetected.Execute(MyGeometry, MouseEvent, OwnerContainer.IsValid() ? OwnerContainer->GetIdentifier() : SPropertyViewer::FHandle(), Item->GetFieldPath());
		}
		else
		{
			return OnDragDetected.Execute(MyGeometry, MouseEvent, SPropertyViewer::FHandle(), TArray<FFieldVariant>());
		}
	}
	return FReply::Unhandled();
}

void FPropertyViewerImpl::HandleSearchChanged(const FText& InFilterText)
{
	if (SearchBoxWidget)
	{
		SearchBoxWidget->SetError(SetRawFilterTextInternal(InFilterText));
	}
}


#if WITH_EDITOR
void FPropertyViewerImpl::HandleBlueprintCompiled()
{
	bool bRemoved = false;
	for (int32 Index = TreeSource.Num()-1; Index >= 0 ; --Index)
	{
		if (TSharedPtr<FContainer> Container = TreeSource[Index]->GetContainer())
		{
			if (!Container->IsValid())
			{
				TreeSource.RemoveAt(Index);
				bRemoved = true;
			}
		}
	}

	for (int32 Index = Containers.Num()-1; Index >= 0; --Index)
	{
		if (!Containers[Index]->IsValid())
		{
			Containers.RemoveAt(Index);
		}
	}

	if (bRemoved)
	{
		TreeWidget->RebuildList();
		if (FilterHandler)
		{
			FilterHandler->RefreshAndFilterTree();
		}
	}
}

void FPropertyViewerImpl::HandleReplaceViewedObjects(const TMap<UObject*, UObject*>& OldToNewObjectMap)
{
	TArray<UObject*> KeyArray;
	OldToNewObjectMap.GenerateKeyArray(KeyArray);

	TArray<TSharedPtr<FContainer>> ItemsToReplace;

	for (TSharedPtr<FContainer> Container : Containers)
	{
		if (Container->GetStruct())
		{
			if (KeyArray.Contains(Container->GetStruct()))
			{
				ItemsToReplace.Add(Container);
			}
		}
	}

	for (TSharedPtr<FContainer> Container : ItemsToReplace)
	{
		Remove(Container->GetIdentifier());
		Containers.Add(Container);
		AddContainerInternal(Container->GetIdentifier(), Container);
	}
}
#endif //WITH_EDITOR


} //namespace

#undef LOCTEXT_NAMESPACE
