// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/PropertyViewer/FieldIconFinder.h"
#include "Framework/PropertyViewer/PropertyPath.h"
#include "Widgets/PropertyViewer/SPropertyViewer.h"
#include "Types/SlateEnums.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"

class ITableRow;
class SSearchBox;
class STableViewBase;
template <typename ItemType>
class STreeView;
template<typename ItemType>
class TreeFilterHandler;
template<typename ItemType>
class TTextFilter;

namespace UE::PropertyViewer
{
class IFieldIterator;
class IFieldExpander;
class SFieldName;
}

namespace UE::PropertyViewer::Private
{

/**
 * FContainer
 */
struct FContainer
{
	FContainer(SPropertyViewer::FHandle InIdentifier, TOptional<FText> DisplayName, const UStruct* ClassToDisplay);
	FContainer(SPropertyViewer::FHandle InIdentifier, TOptional<FText> DisplayName, UObject* InstanceToDisplay);
	FContainer(SPropertyViewer::FHandle InIdentifier, TOptional<FText> DisplayName, const UScriptStruct* Struct, void* Data);

public:
	SPropertyViewer::FHandle GetIdentifier() const
	{
		return Identifier;
	}

	void* GetBuffer() const
	{
		UObject* PinObjectInstance = ObjectInstance.Get();
		return bIsObject ? reinterpret_cast<void*>(PinObjectInstance) : StructInstance;
	}

	const UStruct* GetStruct()
	{
		return Container.Get();
	}

	bool CanEdit() const
	{
		return GetBuffer() != nullptr;
	}

	bool IsInstance() const
	{
		return GetBuffer() != nullptr;
	}
	
	bool IsObjectInstance() const
	{
		return bIsObject;
	}

	UObject* GetObjectInstance() const
	{
		return ObjectInstance.Get();
	}

	bool IsScriptStructInstance() const
	{
		return StructInstance != nullptr;
	}

	void* GetScriptStructInstance() const
	{
		return StructInstance;
	}

	TOptional<FText> GetDisplayName() const
	{
		return DisplayName;
	}

	bool IsValid() const;

private:
	SPropertyViewer::FHandle Identifier;
	TWeakObjectPtr<const UStruct> Container;
	TWeakObjectPtr<UObject> ObjectInstance;
	void* StructInstance = nullptr;
	TOptional<FText> DisplayName;
	bool bIsObject = false;
};


/**
 * FTreeNode
 */
struct FTreeNode : public TSharedFromThis<FTreeNode>
{
public:
	static TSharedRef<FTreeNode> MakeContainer(const TSharedPtr<FContainer>& InContainer, TOptional<FText> InDisplayName);
	static TSharedRef<FTreeNode> MakeField(TSharedPtr<FTreeNode> InParent, const FProperty* Property, TOptional<FText> InDisplayName);
	static TSharedRef<FTreeNode> MakeField(TSharedPtr<FTreeNode> InParent, const UFunction* Function, TOptional<FText> InDisplayName);

private:
	TWeakPtr<FTreeNode> ParentNode;

	//Either a container, property or function
	TWeakPtr<FContainer> Container;
	TWeakObjectPtr<const UStruct> FieldOwner;
	FName PropertyName;
	FName FunctionName;
	bool bIsStructProperty = false;
	bool bIsObjectProperty = false;

	TOptional<FText> OverrideDisplayName;

public:
	TWeakPtr<SFieldName> PropertyWidget;
	TArray<TSharedPtr<FTreeNode>, TInlineAllocator<1>> ChildNodes;
	bool bChildGenerated = false;

public:
	bool IsContainer() const
	{
		return Container.IsValid();
	}

	TSharedPtr<FContainer> GetContainer() const
	{
		return Container.Pin();
	}

	bool IsField() const
	{
		return !PropertyName.IsNone() || !FunctionName.IsNone();
	}
	
	FFieldVariant GetField() const;

	TSharedPtr<FTreeNode> GetParentNode() const
	{
		return ParentNode.Pin();
	}

	FPropertyPath GetPropertyPath() const;
	TArray<FFieldVariant> GetFieldPath() const;

	TSharedPtr<FContainer> GetOwnerContainer() const;

	TOptional<FText> GetOverrideDisplayName() const
	{
		return OverrideDisplayName;
	}

	void GetFilterStrings(TArray<FString>& OutStrings) const;

	struct FNodeReference
	{
		TWeakObjectPtr<UObject> Previous;
		TWeakPtr<FTreeNode> Node;

		FNodeReference(UObject* InPrevious, const TWeakPtr<FTreeNode>& InNode)
			: Previous(InPrevious)
			, Node(InNode)
		{}
	};

	// Returns the object node that was build
	TArray<FNodeReference> BuildChildNodes(IFieldIterator& FieldIterator, IFieldExpander& FieldExpander, bool bSortChildNode);
	void RemoveChild()
	{
		ChildNodes.Reset();
		bChildGenerated = false;
	}

private:
	void BuildChildNodesRecursive(IFieldIterator& FieldIterator, IFieldExpander& FieldExpander, bool bSortChildNode, int32 RecursiveCount, TArray<FNodeReference>& OutTickReference);

	static bool Sort(const TSharedPtr<FTreeNode>& NodeA, const TSharedPtr<FTreeNode>& NodeB);
};


/**
 * FPropertyViewerImpl
 */
class FPropertyViewerImpl : public TSharedFromThis<FPropertyViewerImpl>
{
public:
	FPropertyViewerImpl(const SPropertyViewer::FArguments& InArgs);
	FPropertyViewerImpl(const FPropertyViewerImpl&) = delete;
	FPropertyViewerImpl& operator=(const FPropertyViewerImpl&) = delete;
	~FPropertyViewerImpl();

private:
	TArray<TSharedPtr<FContainer>> Containers;
	TArray<FTreeNode::FNodeReference> NodeReferences;

	TSharedPtr<SSearchBox> SearchBoxWidget;
	TSharedPtr<STreeView<TSharedPtr<FTreeNode>>> TreeWidget;
	TArray<TSharedPtr<FTreeNode>> TreeSource;
	TArray<TSharedPtr<FTreeNode>> FilteredTreeSource;

	using FTextFilter = TTextFilter<TSharedPtr<FTreeNode>>;
	TSharedPtr<FTextFilter> SearchFilter;
	using FTreeFilter = TreeFilterHandler<TSharedPtr<FTreeNode>>;
	TSharedPtr<FTreeFilter> FilterHandler;

	SPropertyViewer::FGetFieldWidget OnGetPreSlot;
	SPropertyViewer::FGetFieldWidget OnGetPostSlot;
	SPropertyViewer::FOnContextMenuOpening OnContextMenuOpening;
	SPropertyViewer::FOnSelectionChanged OnSelectionChanged;
	SPropertyViewer::FOnDoubleClicked OnDoubleClicked;
	SPropertyViewer::FOnDragDetected OnDragDetected;
	SPropertyViewer::FOnGenerateContainer OnGenerateContainer;
	SPropertyViewer::EPropertyVisibility PropertyVisibility;
	TOptional<const FFieldColorSettings> OverrideIconColorSettings;
	bool bSanitizeName = false;
	bool bShowFieldIcon = false;
	bool bUseRows = false;
	bool bSortChildNode = false;
	
	IFieldIterator* FieldIterator = nullptr;
	IFieldExpander* FieldExpander = nullptr;
	INotifyHook* NotifyHook = nullptr;
	bool bOwnFieldIterator = false;
	bool bOwnFieldExpander = false;

public:
	TSharedRef<SWidget> Construct(const SPropertyViewer::FArguments& InArgs);
	void Tick();

	void AddContainer(SPropertyViewer::FHandle Identifier, TOptional<FText> DisplayName, const UStruct* Struct);
	void AddContainerInstance(SPropertyViewer::FHandle Identifier, TOptional<FText> DisplayName, UObject* Object);
	void AddContainerInstance(SPropertyViewer::FHandle Identifier, TOptional<FText> DisplayName, const UScriptStruct* Struct, void* Data);
	void Remove(SPropertyViewer::FHandle Identifier);
	void RemoveAll();
	TArray<SPropertyViewer::FSelectedItem> GetSelectedItems() const;
	void SetRawFilterText(const FText& InFilterText);
	void SetSelection(SPropertyViewer::FHandle Identifier, TArrayView<const FFieldVariant> FieldPath);

private:
	void AddContainerInternal(SPropertyViewer::FHandle Identifier, TSharedPtr<FContainer>& NewContainer);

	TSharedRef<SWidget> CreateSearch();
	TSharedRef<SWidget> CreateTree(bool bHasPreWidget, bool bShowPropertyValue, bool bHasPostWidget, ESelectionMode::Type SelectionMode);

	void HandleSearchChanged(const FText& InFilterText);
	FText SetRawFilterTextInternal(const FText& InFilterText);
	void SetHighlightTextRecursive(const TSharedPtr<FTreeNode>& OwnerNode, const FText& HighlightText);
	void HandleGetFilterStrings(TSharedPtr<FTreeNode> Item, TArray<FString>& OutStrings);

	TSharedRef<ITableRow> HandleGenerateRow(TSharedPtr<FTreeNode> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void HandleGetChildren(TSharedPtr<FTreeNode> InParent, TArray<TSharedPtr<FTreeNode>>& OutChildren);

	TSharedPtr<SWidget> HandleContextMenuOpening();
	void HandleSelectionChanged(TSharedPtr<FTreeNode> Item, ESelectInfo::Type SelectionType);
	void HandleDoubleClick(TSharedPtr<FTreeNode> Item);
	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, const TSharedPtr<FTreeNode> Item);

	TSharedPtr<FTreeNode> FindExistingChild(const TSharedPtr<FTreeNode>& ContainerNode, TArrayView<const FFieldVariant> FieldPath) const;

#if WITH_EDITOR
	void HandleBlueprintCompiled();
	void HandleReplaceViewedObjects(const TMap<UObject*, UObject*>& OldToNewObjectMap);
#endif //WITH_EDITOR
};

} //namespace
