// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/PropertyViewer/SStringPropertyValue.h"

#include "Framework/PropertyViewer/INotifyHook.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"

namespace UE::PropertyViewer
{

TSharedPtr<SWidget> SStringPropertyValue::CreateInstance(const FPropertyValueFactory::FGenerateArgs Args)
{
	return SNew(SBox)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.MinDesiredWidth(100.f)
		[
			SNew(SStringPropertyValue)
			.Path(Args.Path)
			.NotifyHook(Args.NotifyHook)
			.IsEnabled(Args.bCanEditValue)
		];
}


void SStringPropertyValue::Construct(const FArguments& InArgs)
{
	Path = InArgs._Path;
	NotifyHook = InArgs._NotifyHook;

	const FProperty* Property = Path.GetLastProperty();
	if (CastField<const FStrProperty>(Property) || CastField<const FTextProperty>(Property) || CastField<const FNameProperty>(Property))
	{
		if (Property->ArrayDim == 1)
		{
			ChildSlot
			[
				SNew(SEditableTextBox)
				.SelectAllTextWhenFocused(true)
				.Text(this, &SStringPropertyValue::GetText)
				.OnTextCommitted(this, &SStringPropertyValue::OnTextCommitted)
			];
		}
	}
}


FText SStringPropertyValue::GetText() const
{
	if (const void* Container = Path.GetContainerPtr())
	{
		if (const FStrProperty* StrProperty = CastField<const FStrProperty>(Path.GetLastProperty()))
		{
				FString OutString;
				StrProperty->GetValue_InContainer(Container, &OutString);
				return FText::FromString(OutString);
		}
		else if (const FTextProperty* TextProperty = CastField<const FTextProperty>(Path.GetLastProperty()))
		{
				FText OutText;
				TextProperty->GetValue_InContainer(Container, &OutText);
				return OutText;
		}
		else if (const FNameProperty* NameProperty = CastField<const FNameProperty>(Path.GetLastProperty()))
		{
				FName OutText;
				NameProperty->GetValue_InContainer(Container, &OutText);
				return FText::FromName(OutText);
		}
	}
	return FText::GetEmpty();
}


void SStringPropertyValue::OnTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if (void* Container = Path.GetContainerPtr())
	{
		if (const FStrProperty* StrProperty = CastField<const FStrProperty>(Path.GetLastProperty()))
		{
			FString ToSet = InText.ToString();
			if (NotifyHook)
			{
				NotifyHook->OnPreValueChange(Path);
			}
			StrProperty->SetValue_InContainer(Container, ToSet);
			if (NotifyHook)
			{
				NotifyHook->OnPostValueChange(Path);
			}
		}
		else if (const FTextProperty* TextProperty = CastField<const FTextProperty>(Path.GetLastProperty()))
		{
			if (NotifyHook)
			{
				NotifyHook->OnPreValueChange(Path);
			}
			TextProperty->SetValue_InContainer(Container, InText);
			if (NotifyHook)
			{
				NotifyHook->OnPostValueChange(Path);
			}
		}
		else if (const FNameProperty* NameProperty = CastField<const FNameProperty>(Path.GetLastProperty()))
		{
			FName ToSet = *(InText.ToString());
			if (NotifyHook)
			{
				NotifyHook->OnPreValueChange(Path);
			}
			NameProperty->SetValue_InContainer(Container, ToSet);
			if (NotifyHook)
			{
				NotifyHook->OnPostValueChange(Path);
			}
		}
	}
}

} //namespace
