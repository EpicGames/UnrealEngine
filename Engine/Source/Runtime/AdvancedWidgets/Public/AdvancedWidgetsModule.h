// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"


namespace UE::PropertyViewer
{
class FPropertyValueFactory;
}


/**
 * Advanced Widgets module
 */
class FAdvancedWidgetsModule : public IModuleInterface
{
public:
	static ADVANCEDWIDGETS_API FAdvancedWidgetsModule& GetModule();
	static ADVANCEDWIDGETS_API FAdvancedWidgetsModule* GetModulePtr();

	/**
	 * Called right after the plugin DLL has been loaded and the plugin object has been created
	 */
	ADVANCEDWIDGETS_API virtual void StartupModule();

	/**
	 * Called before the plugin is unloaded, right before the plugin object is destroyed.
	 */
	ADVANCEDWIDGETS_API virtual void ShutdownModule();

public:
	ADVANCEDWIDGETS_API UE::PropertyViewer::FPropertyValueFactory& GetPropertyValueFactory() const;

private:
	TUniquePtr<UE::PropertyViewer::FPropertyValueFactory> PropertyValueFactory;
};
