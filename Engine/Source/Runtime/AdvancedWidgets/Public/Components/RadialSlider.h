// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Components/Slider.h"
#include "Components/Widget.h"
#include "Curves/CurveFloat.h"
#include "Styling/SlateTypes.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SWidget.h"

#include "RadialSlider.generated.h"


// Forward Declarations
class SRadialSlider;


/**
 * A simple widget that shows a sliding bar with a handle that allows you to control the value between 0..1.
 *
 * * No Children
 */

UCLASS(MinimalAPI)
class URadialSlider : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	/** The slider value to display. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance, BlueprintSetter = SetValue,  meta = (UIMin="0", UIMax="1"))
	float Value;

	/** A bindable delegate to allow logic to drive the value of the widget */
	UPROPERTY()
	FGetFloat ValueDelegate;
	
	/** Whether the slider should draw it's progress bar from a custom value on the slider */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool bUseCustomDefaultValue;

	/**  The value where the slider should draw it's progress bar from, independent of direction */
	UPROPERTY(EditAnywhere, Category = Appearance, meta = (EditCondition = "bUseCustomDefaultValue", UIMin = "0", UIMax = "1"))
	float CustomDefaultValue;

	/** A curve that defines how the slider should be sampled. Default is linear. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	FRuntimeFloatCurve SliderRange;

	/** Adds text tags to the radial slider at the value's position. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	TArray<float> ValueTags;

	/** The angle at which the Slider Handle will start. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, meta=(ClampMin = "0", ClampMax = "360"), Category = Appearance)
	float SliderHandleStartAngle;

	/** The angle at which the Slider Handle will end. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, meta=(ClampMin = "0", ClampMax = "360"), Category = Appearance)
	float SliderHandleEndAngle;

	/** Rotates radial slider by arbitrary offset to support full gamut of configurations. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, meta=(ClampMin = "0", ClampMax = "360"), Category = Appearance)
	float AngularOffset;

	/** Start and end of the hand as a ratio to the slider radius (so 0.0 to 1.0 is from the slider center to the handle). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	FVector2D HandStartEndRatio;

	/** The progress bar style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Style", meta=( DisplayName="Style" ))
	FSliderStyle WidgetStyle;

	/** The color to draw the slider bar in. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance)
	FLinearColor SliderBarColor;

	/** The color to draw the completed progress of the slider bar in. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	FLinearColor SliderProgressColor;

	/** The color to draw the slider handle in. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance)
	FLinearColor SliderHandleColor;

	/** The color to draw the center background in. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance)
	FLinearColor CenterBackgroundColor;

	/** Whether the handle is interactive or fixed. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance, AdvancedDisplay)
	bool Locked;

	/** Sets new value if mouse position is greater/less than half the step size. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, AdvancedDisplay)
	bool MouseUsesStep;

	/** Sets whether we have to lock input to change the slider value. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance, AdvancedDisplay)
	bool RequiresControllerLock;

	/** The amount to adjust the value by, when using a controller or keyboard */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance, meta=(UIMin="0", UIMax="1"))
	float StepSize;

	/** Should the slider be focusable? */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Interaction")
	bool IsFocusable;

	/** Whether the value is changed when dragging vertically as opposed to along the radial curve.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Behavior")
	bool UseVerticalDrag;

	/** Whether to show the slider handle (thumb).  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	bool ShowSliderHandle;

	/** Whether to show the slider hand.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	bool ShowSliderHand;

	/** Invoked when the mouse is pressed and a capture begins. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnMouseCaptureBeginEvent OnMouseCaptureBegin;

	/** Invoked when the mouse is released and a capture ends. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnMouseCaptureEndEvent OnMouseCaptureEnd;

	/** Invoked when the controller capture begins. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnControllerCaptureBeginEvent OnControllerCaptureBegin;

	/** Invoked when the controller capture ends. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnControllerCaptureEndEvent OnControllerCaptureEnd;

	/** Called when the value is changed by slider or typing. */
	UPROPERTY(BlueprintAssignable, Category="Widget Event")
	FOnFloatValueChangedEvent OnValueChanged;

	/** Gets the current value of the slider. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	ADVANCEDWIDGETS_API float GetValue() const;

	/** Gets the current custom default value of the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	ADVANCEDWIDGETS_API float GetCustomDefaultValue() const;

	/** Get the current raw slider alpha from 0 to 1 */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	ADVANCEDWIDGETS_API float GetNormalizedSliderHandlePosition() const;

	/** Sets the current value of the slider. */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	ADVANCEDWIDGETS_API void SetValue(float InValue);

	/** Sets the current custom default value of the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	ADVANCEDWIDGETS_API void SetCustomDefaultValue(float InValue);

	/** Sets the curve for the slider range*/
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	ADVANCEDWIDGETS_API void SetSliderRange(const FRuntimeFloatCurve& InSliderRange);

	/** Adds value tags to the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	ADVANCEDWIDGETS_API void SetValueTags(const TArray<float>& InValueTags);

	/** Sets the minimum angle of the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	ADVANCEDWIDGETS_API void SetSliderHandleStartAngle(float InValue);

	/** Sets the maximum angle of the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	ADVANCEDWIDGETS_API void SetSliderHandleEndAngle(float InValue);

	/** Sets the Angular Offset for the slider. */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	ADVANCEDWIDGETS_API void SetAngularOffset(float InValue);

	/** Sets the start and end of the hand as a ratio to the slider radius (so 0.0 to 1.0 is from the slider center to the handle). */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	ADVANCEDWIDGETS_API void SetHandStartEndRatio(FVector2D InValue);

	/** Sets the handle to be interactive or fixed */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	ADVANCEDWIDGETS_API void SetLocked(bool InValue);

	/** Sets the amount to adjust the value by, when using a controller or keyboard */
	UFUNCTION(BlueprintCallable, Category="Behavior")
	ADVANCEDWIDGETS_API void SetStepSize(float InValue);

	/** Sets the color of the slider bar */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	ADVANCEDWIDGETS_API void SetSliderBarColor(FLinearColor InValue);

	/** Sets the progress color of the slider bar */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	ADVANCEDWIDGETS_API void SetSliderProgressColor(FLinearColor InValue);

	/** Sets the color of the handle bar */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	ADVANCEDWIDGETS_API void SetSliderHandleColor(FLinearColor InValue);

	/** Sets the color of the slider bar */
	UFUNCTION(BlueprintCallable, Category="Appearance")
	ADVANCEDWIDGETS_API void SetCenterBackgroundColor(FLinearColor InValue);

	/** Set whether the value is changed when dragging vertically as opposed to along the radial curve.  */
	UFUNCTION(BlueprintCallable, Category = "Behavior")
	ADVANCEDWIDGETS_API void SetUseVerticalDrag(bool InUseVerticalDrag);

	/** Whether to show the slider handle (thumb). */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	ADVANCEDWIDGETS_API void SetShowSliderHandle(bool InShowSliderHandle);

	/** Whether to show the slider hand. */
	UFUNCTION(BlueprintCallable, Category = "Appearance")
	ADVANCEDWIDGETS_API void SetShowSliderHand(bool InShowSliderHand);
	
	// UWidget interface
	ADVANCEDWIDGETS_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	// UVisual interface
	ADVANCEDWIDGETS_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	ADVANCEDWIDGETS_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	/** Native Slate Widget */
	TSharedPtr<SRadialSlider> MyRadialSlider;

	// UWidget interface
	ADVANCEDWIDGETS_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	ADVANCEDWIDGETS_API void HandleOnValueChanged(float InValue);
	ADVANCEDWIDGETS_API void HandleOnMouseCaptureBegin();
	ADVANCEDWIDGETS_API void HandleOnMouseCaptureEnd();
	ADVANCEDWIDGETS_API void HandleOnControllerCaptureBegin();
	ADVANCEDWIDGETS_API void HandleOnControllerCaptureEnd();

#if WITH_ACCESSIBILITY
	ADVANCEDWIDGETS_API virtual TSharedPtr<SWidget> GetAccessibleWidget() const override;
#endif

	PROPERTY_BINDING_IMPLEMENTATION(float, Value);
};
