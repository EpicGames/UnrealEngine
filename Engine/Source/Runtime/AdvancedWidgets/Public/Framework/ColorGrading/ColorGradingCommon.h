// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"

namespace UE::ColorGrading
{

/** Callback to get the current FVector4 value */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnGetCurrentVector4Value, FVector4&);

/** Color components that can be controlled by a color grading widget. */
enum class EColorGradingComponent
{
	Red,
	Green,
	Blue,
	Hue,
	Saturation,
	Value,
	Luminance
};

/** Types of color grading modes */
enum class EColorGradingWheelType
{
	/** A wheel that controls a standard color value */
	Standard,

	/** A wheel that applies an offset to colors */
	Offset
};

/** Modes that colors can be displayed in for the color grading panel */
enum class EColorGradingColorDisplayMode
{
	RGB,
	HSV
};

/** Enumerates color picker modes */
enum class EColorGradingModes
{
	Saturation,
	Contrast,
	Gamma,
	Gain,
	Offset,
	Invalid
};

} //namespace
