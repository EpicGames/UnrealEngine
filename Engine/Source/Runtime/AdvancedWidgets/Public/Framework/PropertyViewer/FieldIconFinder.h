// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/ColorList.h"
#include "Styling/SlateColor.h"

struct FSlateBrush;
class FProperty;
class UFunction;

namespace UE::PropertyViewer
{


/** */
struct FFieldColorSettings
{
	/** The default color is used only for types not specifically defined below. */
	FLinearColor DefaultTypeColor = FLinearColor(0.750000f, 0.6f, 0.4f, 1.0f);

	/** Boolean type color */
	FLinearColor BooleanTypeColor = FColorList::Maroon;

	/** Class type color */
	FLinearColor ClassTypeColor = FLinearColor(0.1f, 0.0f, 0.5f, 1.0f);

	/** Enum pin type color */
	FLinearColor EnumTypeColor = FLinearColor(0.0f, 0.160000f, 0.131270f, 1.0f);

	/** Integer pin type color */
	FLinearColor IntTypeColor = FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f);

	/** Floating-point type color */
	FLinearColor FloatTypeColor = FLinearColor(0.357667f, 1.0f, 0.060000f, 1.0f);

	/** Name type color */
	FLinearColor NameTypeColor = FLinearColor(0.607717f, 0.224984f, 1.0f, 1.0f);

	/** Delegate type color */
	FLinearColor DelegateTypeColor = FLinearColor(1.0f, 0.04f, 0.04f, 1.0f);

	/** Object type color */
	FLinearColor ObjectTypeColor = FLinearColor(0.0f, 0.4f, 0.910000f, 1.0f);

	/** Interface type color */
	FLinearColor InterfaceTypeColor = FLinearColor(0.8784f, 1.0f, 0.4f, 1.0f);

	/** String type color */
	FLinearColor StringTypeColor = FLinearColor(1.0f, 0.0f, 0.660537f, 1.0f);

	/** Text type color */
	FLinearColor TextTypeColor = FLinearColor(0.8f, 0.2f, 0.4f, 1.0f);

	/** Default Struct pin type color */
	FLinearColor DefaultStructTypeColor = FColorList::DarkSlateBlue;

	/** Struct pin type color */
	TMap<FString, FLinearColor> StructColors;
};


/** */
class FFieldIconFinder
{
public:
	/** */
	struct FFieldIcon
	{
		const FSlateBrush* Icon = nullptr;
		FSlateColor Color;
	};

	using FFieldIconArray = TArray<FFieldIcon, TInlineAllocator<2>>;

	/** */
	static ADVANCEDWIDGETS_API const FSlateBrush* GetIcon(const UObject* Object);

	/** */
	static ADVANCEDWIDGETS_API FFieldIconArray GetFunctionIcon(const UFunction* Function);
	static ADVANCEDWIDGETS_API FFieldIconArray GetFunctionIcon(const UFunction* Function, const FFieldColorSettings& Settings);

	/** */
	static ADVANCEDWIDGETS_API FFieldIconArray GetPropertyIcon(const FProperty* Property);
	static ADVANCEDWIDGETS_API FFieldIconArray GetPropertyIcon(const FProperty* Property, const FFieldColorSettings& Settings);
};

} //namespace
