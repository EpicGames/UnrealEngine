// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

class FObjectPropertyBase;
class FProperty;
class FStructProperty;
class UFunction;
class UStruct;

namespace UE::PropertyViewer
{

/** */
class IFieldExpander
{
public:
	/** @return the class that should be expended for this instance. The instance can be null. */
	virtual TOptional<const UClass*> CanExpandObject(const FObjectPropertyBase* Property, const UObject* Instance) const = 0;
	/** @return true if the struct property can be expended. */
	virtual bool CanExpandScriptStruct(const FStructProperty* StructProperty) const = 0;
	/** @return the struct that can be expanded from the function. ie. This can be the function itself (to see the arguments) or a return property. */
	virtual TOptional<const UStruct*> GetExpandedFunction(const UFunction* Function) const = 0;
	virtual ~IFieldExpander() = default;

	UE_DEPRECATED(5.2, "CanExpandFunction is deprecated and GetExpandedFunction should be used instead.")
	virtual bool CanExpandFunction(const UFunction* Function) const
	{
		return GetExpandedFunction(Function).IsSet();
	}
};

class FFieldExpander_Default : public IFieldExpander
{
public:
	FFieldExpander_Default() = default;

	enum class EObjectExpandFlag : uint8
	{
		None = 0,
		UseInstanceClass = 1 << 1,
		RequireValidInstance = 1 << 2,
	};


	enum class EFunctionExpand : uint8
	{
		None = 0,
		FunctionProperties,
		FunctionReturnProperty,
	};

	virtual TOptional<const UClass*> CanExpandObject(const FObjectPropertyBase* Property, const UObject* Instance) const override
	{
		const bool bUseInstanceClass = EnumHasAllFlags(ExpandObject, EObjectExpandFlag::UseInstanceClass);
		const bool bOnlyIfInstanceValid = EnumHasAllFlags(ExpandObject, EObjectExpandFlag::RequireValidInstance);
		if (bOnlyIfInstanceValid && !Instance)
		{
			return TOptional<const UClass*>();
		}
		if (bUseInstanceClass && Instance)
		{
			return Instance->GetClass();
		}
		return Property->PropertyClass.Get();
	}

	virtual bool CanExpandScriptStruct(const FStructProperty*) const override
	{
		return bExpandScriptStruct;
	}

	virtual TOptional<const UStruct*> GetExpandedFunction(const UFunction* Function) const override
	{
		if (Function && ExpandFunction == EFunctionExpand::FunctionProperties)
		{
			return Function;
		}
		else if (Function && ExpandFunction == EFunctionExpand::FunctionReturnProperty)
		{
			if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Function->GetReturnProperty()))
			{
				return ObjectProperty->PropertyClass.Get();
			}
		}
		return TOptional<const UStruct*>();
	}

	void SetExpandObject(EObjectExpandFlag InExpandObject)
	{
		ExpandObject = InExpandObject;
	}

	void SetExpandScriptStruct(bool bInExpandScriptStruct)
	{
		bExpandScriptStruct = bInExpandScriptStruct;
	}

	UE_DEPRECATED(5.2, "Use the SetExpandFunction with the EExpandFunctionType")
	void SetExpandFunction(bool bInExpandFunction)
	{
		ExpandFunction = bInExpandFunction ? EFunctionExpand::FunctionProperties : EFunctionExpand::None;
	}

	void SetExpandFunction(EFunctionExpand InExpandFunction)
	{
		ExpandFunction = InExpandFunction;
	}

private:
	EObjectExpandFlag ExpandObject = EObjectExpandFlag::None;
	bool bExpandScriptStruct = false;
	EFunctionExpand ExpandFunction = EFunctionExpand::None;
};
ENUM_CLASS_FLAGS(FFieldExpander_Default::EObjectExpandFlag);

} //namespace
