// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "UObject/Field.h"

class FFieldVariant;
class FProperty;
class UFunction;
class UStruct;

namespace UE::PropertyViewer
{

/** */
class IFieldIterator
{
public:
	UE_DEPRECATED(5.6, "GetFields(const UStruct*) is deprecated. Please use the overload with three arguments: GetFields(const UStruct*, const FName, const UStruct*).")
	virtual TArray<FFieldVariant> GetFields(const UStruct*) const 
	{
		return TArray<FFieldVariant>();
	};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual TArray<FFieldVariant> GetFields(const UStruct* Struct, const FName FieldName, const UStruct* ContainerStruct) const
	{
		return GetFields(Struct);
	};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual ~IFieldIterator() = default;
};

class FFieldIterator_BlueprintVisible : public IFieldIterator
{
	ADVANCEDWIDGETS_API virtual TArray<FFieldVariant> GetFields(const UStruct* Struct, const FName FieldName, const UStruct* ContainerStruct) const override;
};

} //namespace
