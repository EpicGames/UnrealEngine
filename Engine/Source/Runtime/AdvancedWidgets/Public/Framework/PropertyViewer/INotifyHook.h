// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Framework/PropertyViewer/PropertyPath.h"

namespace UE::PropertyViewer
{

/** */
class INotifyHook
{
public:
	virtual void OnPreValueChange(const FPropertyPath& Path) = 0;
	virtual void OnPostValueChange(const FPropertyPath& Path) = 0;
};

} //namespace
