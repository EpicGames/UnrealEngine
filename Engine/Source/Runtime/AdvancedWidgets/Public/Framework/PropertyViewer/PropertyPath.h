// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "Misc/TVariant.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/FieldPath.h"
#include "UObject/Object.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FProperty;
class UObject;
class UScriptStruct;

namespace UE::PropertyViewer
{

struct FPropertyPath
{
public:
	using FPropertyArray = TArray<const FProperty*, TInlineAllocator<1>>;

	FPropertyPath() = default;
	ADVANCEDWIDGETS_API FPropertyPath(UObject* Object, const FProperty* Property);
	ADVANCEDWIDGETS_API FPropertyPath(UObject* Object, FPropertyArray Properties);
	ADVANCEDWIDGETS_API FPropertyPath(const UScriptStruct* ScriptStruct, void* Data, const FProperty* Property);
	ADVANCEDWIDGETS_API FPropertyPath(const UScriptStruct* ScriptStruct, void* Data, FPropertyArray Properties);

public:
	bool HasProperty() const
	{
		return Properties.Num() != 0;
	}

	const FProperty* GetLastProperty() const
	{
		return HasProperty() ? Properties.Last() : nullptr;
	}

	TArrayView<const FProperty* const> GetProperties() const
	{
		return MakeArrayView(Properties);
	}

	ADVANCEDWIDGETS_API void* GetContainerPtr();
	ADVANCEDWIDGETS_API const void* GetContainerPtr() const;

private:
	TWeakObjectPtr<UObject> TopLevelContainer_Object = nullptr;
	TWeakObjectPtr<const UScriptStruct> TopLevelContainer_ScriptStruct = nullptr;
	void* TopLevelContainer_ScriptStructData = nullptr;

	FPropertyArray Properties;
};

} //namespace
