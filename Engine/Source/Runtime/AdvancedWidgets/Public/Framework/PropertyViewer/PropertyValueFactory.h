// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "Framework/PropertyViewer/PropertyPath.h"
#include "Framework/SlateDelegates.h"
#include "UObject/Field.h"
#include "UObject/Class.h"

class SWidget;

namespace UE::PropertyViewer
{
class INotifyHook;

namespace Private
{
class FPropertyValueFactoryImpl;
} // namespace


/** */
class FPropertyValueFactory
{
public:
	struct FHandle
	{
	private:
		friend FPropertyValueFactory;
		int32 Id = 0;
	public:
		bool operator== (FHandle Other) const
		{
			return Id == Other.Id;
		}
		bool operator!= (FHandle Other) const
		{
			return Id != Other.Id;
		}
		bool IsValid() const
		{
			return Id != 0;
		}
	};

	struct FGenerateArgs
	{
		FPropertyPath Path;
		FFieldVariant* Field = nullptr;
		INotifyHook* NotifyHook = nullptr;
		bool bCanEditValue = false;
	};

	/** */
	DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FOnGenerate, FGenerateArgs);

public:
	ADVANCEDWIDGETS_API FPropertyValueFactory();
	FPropertyValueFactory(const FPropertyValueFactory&) = delete;
	FPropertyValueFactory& operator=(const FPropertyValueFactory&) = delete;

	ADVANCEDWIDGETS_API TSharedPtr<SWidget> Generate(FGenerateArgs Args) const;
	ADVANCEDWIDGETS_API TSharedPtr<SWidget> GenerateDefault(FGenerateArgs Args) const;

	ADVANCEDWIDGETS_API bool HasCustomPropertyValue(const FFieldClass* Property) const;
	ADVANCEDWIDGETS_API bool HasCustomPropertyValue(const UStruct* Struct) const;

	ADVANCEDWIDGETS_API FHandle Register(const FFieldClass* Field, FOnGenerate OnGenerate);
	ADVANCEDWIDGETS_API FHandle Register(const UStruct* Struct, FOnGenerate OnGenerate);
	ADVANCEDWIDGETS_API void Unregister(FHandle Handle);

private:
	static ADVANCEDWIDGETS_API FHandle MakeHandle();
	TPimplPtr<Private::FPropertyValueFactoryImpl> Impl;
};

} //namespace
