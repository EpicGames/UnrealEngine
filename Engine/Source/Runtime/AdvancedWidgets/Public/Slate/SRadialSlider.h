// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "Widgets/SRadialSlider.h"

UE_DEPRECATED(5.1, "Including Slate/SRadialSlider.h is deprecated. The location of SRadialSlider has changed. Please include Widgets/SRadialSlider.h instead.")
inline void SRadialSliderHeaderDeprecatedWarning()
{
}

inline void TriggerSRadialSliderHeaderDeprecatedWarning()
{
	SRadialSliderHeaderDeprecatedWarning();
}
