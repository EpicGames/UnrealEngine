// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/PropertyViewer/FieldIconFinder.h"

class ISlateStyle;
class FSlateStyleSet;

namespace UE::AdvancedWidgets
{
/**
 * Style for advanced widgets
 */
class FAdvancedWidgetsStyle
{
public:
	static ADVANCEDWIDGETS_API void Create();
	static ADVANCEDWIDGETS_API void Destroy();
	static ADVANCEDWIDGETS_API const ISlateStyle& Get();

	static const ::UE::PropertyViewer::FFieldColorSettings& GetColorSettings()
	{
		return ColorSettings;
	}

private:
	/** Singleton instances of this style. */
	static ADVANCEDWIDGETS_API TUniquePtr<FSlateStyleSet> Instance;

	/**  */
	static ::UE::PropertyViewer::FFieldColorSettings ColorSettings;
};

} // namespace
