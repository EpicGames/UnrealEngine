// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateTypes.h"

#include "ColorGradingSpinBoxStyle.generated.h"

/**
 * Represents the appearance of a color grading spin box
 */
USTRUCT(BlueprintType)
struct FColorGradingSpinBoxStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	ADVANCEDWIDGETS_API FColorGradingSpinBoxStyle();

	virtual ~FColorGradingSpinBoxStyle() = default;

	ADVANCEDWIDGETS_API virtual void GetResources(TArray< const FSlateBrush* >& OutBrushes) const override;

	static ADVANCEDWIDGETS_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static ADVANCEDWIDGETS_API const FColorGradingSpinBoxStyle& GetDefault();

	/** Brush used to draw the border of the spinbox */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush BorderBrush;
	FColorGradingSpinBoxStyle& SetBorderBrush(const FSlateBrush& InBorderBrush) { BorderBrush = InBorderBrush; return *this; }

	/** Brush used to draw the border of the spinbox when it's in active use by the user */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush ActiveBorderBrush;
	FColorGradingSpinBoxStyle& SetActiveBorderBrush(const FSlateBrush& InActiveBorderBrush) { ActiveBorderBrush = InActiveBorderBrush; return *this; }

	/** Brush used to draw the border of the spinbox when it's hovered over */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush HoveredBorderBrush;
	FColorGradingSpinBoxStyle& SetHoveredBorderBrush(const FSlateBrush& InHoveredBorderBrush) { HoveredBorderBrush = InHoveredBorderBrush; return *this; }

	/** Brush used to draw the selector indicating the current value */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush SelectorBrush;
	FColorGradingSpinBoxStyle& SetSelectorBrush(const FSlateBrush& InSelectorBrush) { SelectorBrush = InSelectorBrush; return *this; }

	/** Width of the selector */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float SelectorWidth;
	FColorGradingSpinBoxStyle& SetSelectorWidth(float InSelectorWidth) { SelectorWidth = InSelectorWidth; return *this; }

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		BorderBrush.UnlinkColors();
		HoveredBorderBrush.UnlinkColors();
		ActiveBorderBrush.UnlinkColors();
		SelectorBrush.UnlinkColors();
	}
};
