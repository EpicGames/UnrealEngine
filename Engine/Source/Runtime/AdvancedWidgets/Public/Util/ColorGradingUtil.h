// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/ColorGrading/ColorGradingCommon.h"

namespace UE::ColorGrading
{

/** Get the color grading component typically associated with the given display mode and component index. */
EColorGradingComponent ADVANCEDWIDGETS_API GetColorGradingComponent(EColorGradingColorDisplayMode DisplayMode, int Index);

}
