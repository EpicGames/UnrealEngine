// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "PropertyHandle.h"

/** Convenience wrapper for an IPropertyHandle that lets us track when we're using this handle to set the property's vector value. */
class FTrackedVector4PropertyHandle
{
public:
	ADVANCEDWIDGETS_API FTrackedVector4PropertyHandle();
	ADVANCEDWIDGETS_API FTrackedVector4PropertyHandle(TWeakPtr<IPropertyHandle> InHandle);

	/** Get the underlying property handle. */
	ADVANCEDWIDGETS_API TSharedPtr<IPropertyHandle> GetHandle() const;

	/** Set the property's vector value */
	ADVANCEDWIDGETS_API FPropertyAccess::Result SetValue(const FVector4& InValue, EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags);

	/** Get the property's vector value */
	ADVANCEDWIDGETS_API FPropertyAccess::Result GetValue(FVector4& OutValue) const;

	/**
	 * @return Whether we're currently inside a call to SetValue
	 */
	ADVANCEDWIDGETS_API bool IsSettingValue() const;

	/**
	 * @return Whether or not the handle points to a valid property node. This can be true but GetProperty may still return null
	 */
	ADVANCEDWIDGETS_API bool IsValidHandle() const;

private:
	/** The underlying handle to the property */
	TWeakPtr<IPropertyHandle> Handle;

	/** Whether we're currently changing the property using SetValue */
	bool bIsSettingValue = false;
};

#endif
