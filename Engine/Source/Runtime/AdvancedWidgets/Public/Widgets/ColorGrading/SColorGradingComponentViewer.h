// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/ColorGrading/ColorGradingCommon.h"

#define UE_API ADVANCEDWIDGETS_API

namespace UE::ColorGrading
{

class SColorGradingComponentSpinBox;

/**
 * A slider displayed in tandem with a color grading wheel.
 * Allows mouse-based control or direct input via a textbox.
 * Shows a gradient representing the controlled color component.
 */
class SColorGradingComponentViewer
	: public SCompoundWidget
{
public:

	/** Notification for numeric value change */
	DECLARE_DELEGATE_OneParam(FOnValueChanged, float);

	/** Notification when the max/min spinner values are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	DECLARE_DELEGATE_FourParams(FOnDynamicSliderMinMaxValueChanged, float, TWeakPtr<SWidget>, bool, bool);

	SLATE_BEGIN_ARGS(SColorGradingComponentViewer)
		: _Value(0.f)
		, _MinValue(0.f)
		, _MaxValue(2.f)
		, _MinFractionalDigits(DefaultMinFractionalDigits)
		, _MaxFractionalDigits(DefaultMaxFractionalDigits)
		, _Delta(0.f)
		, _ShiftMultiplier(10.f)
		, _CtrlMultiplier(0.1f)
		, _SupportDynamicSliderMaxValue(false)
		, _SupportDynamicSliderMinValue(false)
		, _SliderExponent(1.f)
		, _OnValueChanged()
		, _OnQueryCurrentColor()
		, _UseCompactDisplay(false)
		, _AllowSpin(true)
		{}

		/** The value to display */
		SLATE_ATTRIBUTE(TOptional<float>, Value)
		/** The component being displayed */
		SLATE_ATTRIBUTE(EColorGradingComponent, Component)
		/** The mode of the associated color grading wheel */
		SLATE_ATTRIBUTE(EColorGradingModes, ColorGradingMode)
		/** The minimum value that can be entered into the text edit box */
		SLATE_ATTRIBUTE(TOptional<float>, MinValue)
		/** The maximum value that can be entered into the text edit box */
		SLATE_ATTRIBUTE(TOptional<float>, MaxValue)
		/** The minimum value that can be specified by using the slider, defaults to MinValue */
		SLATE_ATTRIBUTE(TOptional<float>, MinSliderValue)
		/** The maximum value that can be specified by using the slider, defaults to MaxValue */
		SLATE_ATTRIBUTE(TOptional<float>, MaxSliderValue)
		/** The minimum fractional digits the spin box displays, defaults to 1 */
		SLATE_ATTRIBUTE(TOptional<int32>, MinFractionalDigits)
		/** The maximum fractional digits the spin box displays, defaults to 3 */
		SLATE_ATTRIBUTE(TOptional<int32>, MaxFractionalDigits)
		/** Delta to increment the value as the slider moves.  If not specified will determine automatically */
		SLATE_ATTRIBUTE(float, Delta)
		/** Multiplier to use when shift is held down */
		SLATE_ATTRIBUTE(float, ShiftMultiplier)
		/** Multiplier to use when ctrl is held down */
		SLATE_ATTRIBUTE(float, CtrlMultiplier)
		/** If we're an unbounded spinbox, what value do we divide mouse movement by before multiplying by Delta. Requires Delta to be set. */
		SLATE_ATTRIBUTE(int32, LinearDeltaSensitivity)
		/** Tell us if we want to support dynamically changing of the max value using alt */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMaxValue)
		/** Tell us if we want to support dynamically changing of the min value using alt */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMinValue)
		/** Called right after the max slider value is changed (only relevant if SupportDynamicSliderMaxValue is true) */
		SLATE_EVENT(FOnDynamicSliderMinMaxValueChanged, OnDynamicSliderMaxValueChanged)
		/** Called right after the min slider value is changed (only relevant if SupportDynamicSliderMinValue is true) */
		SLATE_EVENT(FOnDynamicSliderMinMaxValueChanged, OnDynamicSliderMinValueChanged)
		/** Use exponential scale for the slider */
		SLATE_ATTRIBUTE(float, SliderExponent)
		/** When use exponential scale for the slider which is the neutral value */
		SLATE_ATTRIBUTE(float, SliderExponentNeutralValue)
		/** Step to increment or decrement the value by when scrolling the mouse wheel. If not specified will determine automatically */
		SLATE_ATTRIBUTE(TOptional<float>, WheelStep)
		/** Called when the value is changed by slider or typing */
		SLATE_EVENT(FOnValueChanged, OnValueChanged)
		/** Called right before the slider begins to move */
		SLATE_EVENT(FSimpleDelegate, OnBeginSliderMovement)
		/** Called right after the slider handle is released by the user */
		SLATE_EVENT(FOnValueChanged, OnEndSliderMovement)
		/** Callback to get the current FVector4 color value (used to update the background gradient) */
		SLATE_EVENT(FOnGetCurrentVector4Value, OnQueryCurrentColor)
		/** Provide custom type conversion functionality to the spin box */
		SLATE_ARGUMENT(TSharedPtr<INumericTypeInterface<float>>, TypeInterface)
		/** If true, reduce padding and hide the gradient spinbox, leaving only the numeric entry. */
		SLATE_ARGUMENT(bool, UseCompactDisplay)
		/** Whether or not the user should be able to change the value by dragging with the mouse cursor */
		SLATE_ARGUMENT(bool, AllowSpin)

	SLATE_END_ARGS()

	SColorGradingComponentViewer() = default;

	UE_API virtual ~SColorGradingComponentViewer();

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct(const FArguments& InArgs);

	/** Get the maximum slider value */
	UE_API float GetMaxSliderValue() const;

	/** Get the minimum slider value */
	UE_API float GetMinSliderValue() const;

private:

	/** Get the value as a non-optional float */
	UE_API float GetValue() const;

	/** Get the component's name as shown on a short label */
	UE_API FText GetComponentLabelText() const;

	/** Get the component's name as shown in a tooltip */
	UE_API FText GetComponentToolTipText() const;

	/** The default minimum fractional digits */
	static constexpr int32 DefaultMinFractionalDigits = 1;

	/** The default maximum fractional digits */
	static constexpr int32 DefaultMaxFractionalDigits = 3;

	TAttribute<TOptional<float>> MinValue;
	TAttribute<TOptional<float>> MaxValue;
	TAttribute<TOptional<float>> MinSliderValue;
	TAttribute<TOptional<float>> MaxSliderValue;
	TAttribute<TOptional<float>> OptionalValue;
	TSharedPtr<SNumericEntryBox<float>> NumericEntryBox;
	TSharedPtr<SColorGradingComponentSpinBox> GradientSpinBox;
	TAttribute<EColorGradingComponent> Component;
};

} //namespace

#undef UE_API
