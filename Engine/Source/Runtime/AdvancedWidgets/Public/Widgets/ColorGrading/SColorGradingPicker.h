// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Framework/ColorGrading/ColorGradingCommon.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SNumericEntryBox.h"

namespace UE::ColorGrading
{

/**
 * Class for placing a color picker. If all you need is a standalone color picker,
 * use the functions OpenColorGradingWheel and DestroyColorGradingWheel, since they hold a static
 * instance of the color picker.
 */
class SColorGradingPicker
	: public SCompoundWidget
{
public:
	/** Notification when the max/min spinner values are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnNumericEntryBoxDynamicSliderMinMaxValueChanged, float, TWeakPtr<SWidget>, bool, bool);

	// Delegate called when the widget Color Data changed
	DECLARE_DELEGATE_TwoParams(FOnColorGradingPickerValueChanged, FVector4&, bool);


	SLATE_BEGIN_ARGS(SColorGradingPicker)
		: _AllowSpin(true)
		, _SupportDynamicSliderMaxValue(false)
		, _SupportDynamicSliderMinValue(false)
		, _DesiredWheelSize()
		, _MainDelta(0.01f)
		, _MainShiftMultiplier(10.f)
		, _MainCtrlMultiplier(0.1f)
		, _ColorGradingModes(EColorGradingModes::Saturation)
		, _OnColorCommitted()
		, _OnQueryCurrentColor()
		{ }

		SLATE_ARGUMENT(TOptional<float>, ValueMin)
		SLATE_ARGUMENT(TOptional<float>, ValueMax)
		SLATE_ARGUMENT(TOptional<float>, SliderValueMin)
		SLATE_ARGUMENT(TOptional<float>, SliderValueMax)
		SLATE_ATTRIBUTE(bool, AllowSpin)

		/** Tell us if we want to support dynamically changing of the max value using ctrl */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMaxValue)
		/** Tell us if we want to support dynamically changing of the min value using ctrl */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMinValue)

		SLATE_ATTRIBUTE(int32, DesiredWheelSize)

		SLATE_ARGUMENT(float, MainDelta)

		/** Multiplier to use when shift is held down */
		SLATE_ARGUMENT(float, MainShiftMultiplier)
		/** Multiplier to use when ctrl is held down */
		SLATE_ARGUMENT(float, MainCtrlMultiplier)

		SLATE_ARGUMENT_DEPRECATED(int32, MainShiftMouseMovePixelPerDelta, 5.4, "Shift Mouse Move Pixel Per Delta is deprecated and incrementing by a fixed delta per pixel is no longer supported. Please use ShiftMultiplier and CtrlMultiplier which will multiply the step per mouse move")

		SLATE_ARGUMENT(EColorGradingModes, ColorGradingModes)

		/** The event called when the color is committed */
		SLATE_EVENT(FOnColorGradingPickerValueChanged, OnColorCommitted)

		/** Callback to get the current FVector4 value */
		SLATE_EVENT(FOnGetCurrentVector4Value, OnQueryCurrentColor)

		/** Called right before the slider begins to move */
		SLATE_EVENT(FSimpleDelegate, OnBeginSliderMovement)

		/** Called right after the slider handle is released by the user */
		SLATE_EVENT(FSimpleDelegate, OnEndSliderMovement)

		/** Called when the mouse captures starts on the color wheel */
		SLATE_EVENT(FSimpleDelegate, OnBeginMouseCapture)

		/** Called when the mouse captures ends for the color wheel */
		SLATE_EVENT(FSimpleDelegate, OnEndMouseCapture)

	SLATE_END_ARGS()

	/**	Destructor. */
	ADVANCEDWIDGETS_API ~SColorGradingPicker();

public:

	/**
	 * Construct the widget
	 *
	 * @param InArgs Declaration from which to construct the widget.
	 */
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs);

	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged& GetOnNumericEntryBoxDynamicSliderMaxValueChangedDelegate() { return OnNumericEntryBoxDynamicSliderMaxValueChanged; }
	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged& GetOnNumericEntryBoxDynamicSliderMinValueChangedDelegate() { return OnNumericEntryBoxDynamicSliderMinValueChanged; }

	/** Callback when the max/min spinner value are changed (only apply if SupportDynamicSliderMaxValue or SupportDynamicSliderMinValue are true) */
	ADVANCEDWIDGETS_API void OnDynamicSliderMaxValueChanged(float NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfHigher);
	ADVANCEDWIDGETS_API void OnDynamicSliderMinValueChanged(float NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfLower);

protected:

	ADVANCEDWIDGETS_API void TransformLinearColorRangeToColorGradingRange(FVector4& VectorValue) const;
	ADVANCEDWIDGETS_API void TransformColorGradingRangeToLinearColorRange(FVector4& VectorValue) const;
	ADVANCEDWIDGETS_API void TransformColorGradingRangeToLinearColorRange(float& FloatValue);

	ADVANCEDWIDGETS_API TOptional<float> OnGetMainValue() const;
	ADVANCEDWIDGETS_API void OnMainValueChanged(float InValue, bool ShouldCommitValueChanges);
	ADVANCEDWIDGETS_API void OnMainValueCommitted(float InValue, ETextCommit::Type CommitType);

	ADVANCEDWIDGETS_API FLinearColor GetCurrentLinearColor();

	ADVANCEDWIDGETS_API bool IsEntryBoxEnabled() const;

	// Callback for value changes in the color spectrum picker.
	ADVANCEDWIDGETS_API void HandleCurrentColorValueChanged(const FLinearColor& NewValue, bool ShouldCommitValueChanges);

	ADVANCEDWIDGETS_API void HandleColorWheelMouseCaptureBegin(const FLinearColor& InValue);
	ADVANCEDWIDGETS_API void HandleColorWheelMouseCaptureEnd(const FLinearColor& InValue);

	ADVANCEDWIDGETS_API void OnBeginSliderMovement();
	ADVANCEDWIDGETS_API void OnEndSliderMovement(float NewValue);
	ADVANCEDWIDGETS_API void AdjustRatioValue(FVector4& NewValue);

	bool bIsMouseDragging;
	FVector4 StartDragRatio;

	float SliderValueMin;
	float SliderValueMax;
	float MainDelta;
	float MainShiftMultiplier;
	float MainCtrlMultiplier;
	EColorGradingModes ColorGradingModes;

	TSharedPtr<SNumericEntryBox<float>> NumericEntryBoxWidget;

	/** Invoked when a new value is selected on the color wheel */
	FOnColorGradingPickerValueChanged OnColorCommitted;

	FOnGetCurrentVector4Value OnQueryCurrentColor;
	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged OnNumericEntryBoxDynamicSliderMaxValueChanged;
	FOnNumericEntryBoxDynamicSliderMinMaxValueChanged OnNumericEntryBoxDynamicSliderMinValueChanged;

	FSimpleDelegate ExternalBeginSliderMovementDelegate;
	FSimpleDelegate ExternalEndSliderMovementDelegate;
	FSimpleDelegate ExternalBeginMouseCaptureDelegate;
	FSimpleDelegate ExternalEndMouseCaptureDelegate;
};

} //namespace