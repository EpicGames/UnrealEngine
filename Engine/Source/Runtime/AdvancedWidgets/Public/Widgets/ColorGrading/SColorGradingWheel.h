// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Framework/SlateDelegates.h"

class UMaterial;

namespace UE::ColorGrading
{

/**
 * Implements the color wheel widget.
 */
class SColorGradingWheel
	: public SLeafWidget, public FGCObject
{
	SLATE_DECLARE_WIDGET_API(SColorGradingWheel, SLeafWidget, ADVANCEDWIDGETS_API)

public:

	DECLARE_DELEGATE_OneParam(FOnColorGradingWheelMouseCapture, const FLinearColor&);
	DECLARE_DELEGATE_OneParam(FOnColorGradingWheelValueChanged, const FLinearColor&);

	SLATE_BEGIN_ARGS(SColorGradingWheel)
		: _SelectedColor()
		, _DesiredWheelSize()
		, _ExponentDisplacement()
		, _OnMouseCaptureBegin()
		, _OnMouseCaptureEnd()
		, _OnValueChanged()
		{ }

		/** The current color selected by the user. */
		SLATE_ATTRIBUTE(FLinearColor, SelectedColor)

		SLATE_ATTRIBUTE(int32, DesiredWheelSize)

		SLATE_ATTRIBUTE(float, ExponentDisplacement)

		/** Invoked when the mouse is pressed and a capture begins. */
		SLATE_EVENT(FOnColorGradingWheelMouseCapture, OnMouseCaptureBegin)

		/** Invoked when the mouse is released and a capture ends. */
		SLATE_EVENT(FOnColorGradingWheelMouseCapture, OnMouseCaptureEnd)

		/** Invoked when a new value is selected on the color wheel. */
		SLATE_EVENT(FOnColorGradingWheelValueChanged, OnValueChanged)

	SLATE_END_ARGS()

public:
	ADVANCEDWIDGETS_API SColorGradingWheel();

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs);

public:

	// SWidget overrides

	ADVANCEDWIDGETS_API virtual FVector2D ComputeDesiredSize(float) const override;
	ADVANCEDWIDGETS_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	ADVANCEDWIDGETS_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	ADVANCEDWIDGETS_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	ADVANCEDWIDGETS_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	ADVANCEDWIDGETS_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

protected:

	/**
	 * Calculates the position of the color selection indicator.
	 *
	 * @return The position relative to the widget.
	 */
	ADVANCEDWIDGETS_API UE::Slate::FDeprecateVector2DResult CalcRelativePositionFromCenter() const;

	/**
	 * Performs actions according to mouse click / move
	 *
	 * @return	True if the mouse action occurred within the color wheel radius
	 */
	ADVANCEDWIDGETS_API bool ProcessMouseAction(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, bool bProcessWhenOutsideColorWheel);

	/** */
	ADVANCEDWIDGETS_API void SetSelectedColorAttribute(TAttribute<FLinearColor> InSelectedColor);

	/** */
	ADVANCEDWIDGETS_API void SetDesiredWheelSizeAttribute(TAttribute<int32> InDesiredWheelSize);

	/** */
	ADVANCEDWIDGETS_API void SetExponentDisplacementAttribute(TAttribute<float> InExponentDisplacement);

	/** Get the actual size of the wheel, taking into account the available space */
	ADVANCEDWIDGETS_API FVector2f GetActualSize(const FGeometry& MyGeometry) const;

	/** @return an attribute reference of SelectedColor */
	TSlateAttributeRef<FLinearColor> GetSelectedColorAttribute() const { return TSlateAttributeRef<FLinearColor>(SharedThis(this), SelectedColorAttribute); }

	/** @return an attribute reference of DesiredWheelSize */
	TSlateAttributeRef<int32> GetDesiredWheelSizeAttribute() const { return TSlateAttributeRef<int32>(SharedThis(this), DesiredWheelSizeAttribute); }

	/** @return an attribute reference of ExponentDisplacement */
	TSlateAttributeRef<float> GetExponentDisplacementAttribute() const { return TSlateAttributeRef<float>(SharedThis(this), ExponentDisplacementAttribute); }

	/** The material to use for the wheel's background. */
	TObjectPtr<UMaterial> BackgroundMaterial;

	/** The brush to use for the wheel's background. */
	TSharedPtr<FSlateBrush> BackgroundImage;

	/** The cross to show on top of the color wheel background. */
	const FSlateBrush* CrossImage;

	/** The color selector image to show. */
	const FSlateBrush* SelectorImage;

	/** Invoked when the mouse is pressed and a capture begins. */
	FOnColorGradingWheelMouseCapture OnMouseCaptureBegin;

	/** Invoked when the mouse is let up and a capture ends. */
	FOnColorGradingWheelMouseCapture OnMouseCaptureEnd;

	/** Invoked when a new value is selected on the color wheel. */
	FOnColorGradingWheelValueChanged OnValueChanged;

private:

	//~ Begin FGCObject
	virtual FString GetReferencerName() const override;
	virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;
	//~ End FGCObject

	/** The angle in degrees (clockwise from due east) at which hue 0/red is represented on the wheel. */
	static constexpr float HueAngleOffset = -120.f / 180.f * PI;

	/** The current color selected by the user. */
	TSlateAttribute<FLinearColor> SelectedColorAttribute;

	TSlateAttribute<int32> DesiredWheelSizeAttribute;
	TSlateAttribute<float> ExponentDisplacementAttribute;

	/** Flags used to check if the SlateAttribute is set. */
	union
	{
		struct
		{
			uint8 bIsAttributeDesiredWheelSizeSet : 1;
			uint8 bIsAttributeExponentDisplacementSet : 1;
		};
		uint8 Union_IsAttributeSet;
	};
};

} //namespace
