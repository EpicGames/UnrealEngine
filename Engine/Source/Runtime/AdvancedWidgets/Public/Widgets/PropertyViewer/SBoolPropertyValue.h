// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/PropertyViewer/PropertyPath.h"
#include "Framework/PropertyViewer/PropertyValueFactory.h"

enum class ECheckBoxState : uint8;

namespace UE::PropertyViewer
{

class INotifyHook;

/** */
class SBoolPropertyValue : public SCompoundWidget
{
public:
	static ADVANCEDWIDGETS_API TSharedPtr<SWidget> CreateInstance(const FPropertyValueFactory::FGenerateArgs Args);

public:
	SLATE_BEGIN_ARGS(SBoolPropertyValue) {}
		SLATE_ARGUMENT(FPropertyPath, Path);
		SLATE_ARGUMENT(INotifyHook*, NotifyHook);
	SLATE_END_ARGS()

	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs);

private:
	ECheckBoxState HandleIsChecked() const;
	void HandleCheckStateChanged(ECheckBoxState NewState);

	FPropertyPath Path;
	INotifyHook* NotifyHook = nullptr;
};

} //namespace
