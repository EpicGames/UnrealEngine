// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/PropertyViewer/PropertyPath.h"
#include "Framework/PropertyViewer/PropertyValueFactory.h"


namespace UE::PropertyViewer
{

/** */
class SDefaultPropertyValue : public SCompoundWidget
{
public:
	static ADVANCEDWIDGETS_API TSharedPtr<SWidget> CreateInstance(const FPropertyValueFactory::FGenerateArgs Args);

public:
	SLATE_BEGIN_ARGS(SDefaultPropertyValue) {}
		SLATE_ARGUMENT(FPropertyPath, Path);
	SLATE_END_ARGS()

	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs);

private:
	FText GetText() const;

	FPropertyPath Path;
};

} //namespace
