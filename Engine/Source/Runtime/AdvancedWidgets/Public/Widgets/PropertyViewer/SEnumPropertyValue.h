// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/PropertyViewer/PropertyPath.h"
#include "Framework/PropertyViewer/PropertyValueFactory.h"


namespace UE::PropertyViewer
{

class INotifyHook;

/** */
class SEnumPropertyValue : public SCompoundWidget
{
public:
	static ADVANCEDWIDGETS_API TSharedPtr<SWidget> CreateInstance(const FPropertyValueFactory::FGenerateArgs Args);

public:
	SLATE_BEGIN_ARGS(SEnumPropertyValue) {}
		SLATE_ARGUMENT(FPropertyPath, Path);
		SLATE_ARGUMENT(INotifyHook*, NotifyHook);
	SLATE_END_ARGS()

	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs);

private:
	FText GetText() const;
	TSharedRef<SWidget> OnGetMenuContent();
	//bool IsEnumEntryChecked(int32 Index) const;
	void SetEnumEntry(int32 Index);
	int32 GetCurrentValue() const;

	TWeakObjectPtr<const UEnum> EnumType;
	FPropertyPath Path;
	INotifyHook* NotifyHook = nullptr;
};

} //namespace
