// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/PropertyViewer/FieldIconFinder.h"


namespace UE::PropertyViewer
{

/** */
class SFieldIcon : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFieldIcon) {}
		SLATE_ARGUMENT(TOptional<const FFieldColorSettings>, OverrideColorSettings);
	SLATE_END_ARGS()

	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const UClass* Class);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const UScriptStruct* Struct);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const FProperty* Property);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const UFunction* FunctionToDisplay);
};

} //namespace
