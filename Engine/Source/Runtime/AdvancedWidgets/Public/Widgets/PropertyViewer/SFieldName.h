// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/PropertyViewer/FieldIconFinder.h"
#include "UObject/Field.h"

class IToolTip;
class STextBlock;

namespace UE::PropertyViewer
{

/** */
class SFieldName : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFieldName)
	{}
		/** Show the class icon or the color that correspond to the property type. */
		SLATE_ARGUMENT_DEFAULT(bool, bShowIcon) = true;
		/** Sanitize the property name. */
		SLATE_ARGUMENT_DEFAULT(bool, bSanitizeName) = false;
		/** Override the DisplayName of the container name. */
		SLATE_ARGUMENT(TOptional<FText>, OverrideDisplayName);
		/** Override the color settings of the icon. */
		SLATE_ARGUMENT(TOptional<const FFieldColorSettings>, OverrideIconColorSettings);
		/** The current highlighted text. */
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const UClass* Class);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const UScriptStruct* Struct);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const FProperty* Property);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const UFunction* FunctionToDisplay);

	ADVANCEDWIDGETS_API void SetHighlightText(TAttribute<FText> InHighlightText);

private:
	FText GetToolTipText() const;
	void Construct(const FArguments& InArgs, const FText& DisplayName, TSharedPtr<SWidget> Icon);

	TSharedPtr<STextBlock> NameBlock;
	FFieldVariant Field;
};

} //namespace
