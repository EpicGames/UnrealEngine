// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/PropertyViewer/PropertyPath.h"
#include "Framework/PropertyViewer/PropertyValueFactory.h"


namespace UE::PropertyViewer
{

class INotifyHook;

/** */
class SNumericPropertyValue : public SCompoundWidget
{
public:
	static ADVANCEDWIDGETS_API TSharedPtr<SWidget> CreateInstance(const FPropertyValueFactory::FGenerateArgs Args);

public:
	SLATE_BEGIN_ARGS(SNumericPropertyValue) {}
		SLATE_ARGUMENT(FPropertyPath, Path);
		SLATE_ARGUMENT(INotifyHook*, NotifyHook);
	SLATE_END_ARGS()

	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs);

private:
	void OnEndSliderMovement_uint64(uint64 Value);
	void OnEndSliderMovement_int64(int64 Value);
	void OnEndSliderMovement_double(double Value);
	uint64 GetCurrentValue_uint64() const;
	int64 GetCurrentValue_int64() const;
	double GetCurrentValue_double() const;
	void OnValueChanged_uint64(uint64 Value);
	void OnValueChanged_int64(int64 Value);
	void OnValueChanged_double(double Value);
	void OnValueCommitted_uint64(uint64 Value, ETextCommit::Type CommitInfo);
	void OnValueCommitted_int64(int64 Value, ETextCommit::Type CommitInfo);
	void OnValueCommitted_double(double Value, ETextCommit::Type CommitInfo);

private:
	FPropertyPath Path;
	INotifyHook* NotifyHook = nullptr;
};

} //namespace
