// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Framework/PropertyViewer/PropertyPath.h"
#include "Framework/PropertyViewer/PropertyValueFactory.h"

class INotifyHook;

namespace UE::PropertyViewer
{

	/** */
	class SObjectPropertyValue : public SCompoundWidget
	{
	public:
		static ADVANCEDWIDGETS_API TSharedPtr<SWidget> CreateInstance(const FPropertyValueFactory::FGenerateArgs Args);

	public:
		SLATE_BEGIN_ARGS(SObjectPropertyValue) {}
			SLATE_ARGUMENT(FPropertyPath, Path);
			SLATE_ARGUMENT(INotifyHook*, NotifyHook);
		SLATE_END_ARGS()

		ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs);

	private:
		FPropertyPath Path;
		INotifyHook* NotifyHook = nullptr;

#if WITH_EDITOR
		FString GetObjectPath() const;

		void OnObjectSelect(const FAssetData& InAsset);
#endif
	};

} //namespace
