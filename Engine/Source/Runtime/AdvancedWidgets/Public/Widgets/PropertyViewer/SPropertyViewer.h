// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/PropertyViewer/FieldIconFinder.h"
#include "Framework/Views/ITypedTableView.h"
#include "Templates/TypeHash.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/StructOnScope.h"

namespace UE::PropertyViewer
{

class IFieldIterator;
class IFieldExpander;
class INotifyHook;

namespace Private
{
class FPropertyViewerImpl;
}

/** */
class SPropertyViewer : public SCompoundWidget
{
private:
	using Super = SCompoundWidget;

public:
	class FHandle
	{
	private:
		friend SPropertyViewer;
		int32 Id = 0;
	public:
		bool operator==(FHandle Other) const
		{
			return Id == Other.Id;
		}
		bool operator!=(FHandle Other) const
		{
			return Id != Other.Id;
		}
		bool IsValid() const
		{
			return Id != 0;
		}
		friend uint32 GetTypeHash(const FHandle& Other)
		{
			return ::GetTypeHash(Other.Id);
		}
	};

	enum class EPropertyVisibility
	{
		Hidden,
		Visible,
		Editable,
	};

	struct FSelectedItem
	{
		FHandle Handle;
		TArray<TArray<FFieldVariant>> Fields;
		bool bIsContainerSelected = false;
	};

public:
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedPtr<SWidget>, FGetFieldWidget, FHandle, TArrayView<const FFieldVariant>);
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedPtr<SWidget>, FOnContextMenuOpening, FHandle, TArrayView<const FFieldVariant>);
	DECLARE_DELEGATE_ThreeParams(FOnSelectionChanged, FHandle, TArrayView<const FFieldVariant>, ESelectInfo::Type);
	DECLARE_DELEGATE_TwoParams(FOnDoubleClicked, FHandle, TArrayView<const FFieldVariant>);
	DECLARE_DELEGATE_RetVal_FourParams(FReply, FOnDragDetected, const FGeometry&, const FPointerEvent&, FHandle, TArrayView<const FFieldVariant>);
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<SWidget>, FOnGenerateContainer, FHandle, TOptional<FText> DisplayName);

	SLATE_BEGIN_ARGS(SPropertyViewer)
	{}
		/** Allow to edit the instance property. */
		SLATE_ARGUMENT_DEFAULT(EPropertyVisibility, PropertyVisibility) = EPropertyVisibility::Hidden;
		/** Sanitize the field and container name. */
		SLATE_ARGUMENT_DEFAULT(bool, bSanitizeName) = false;
		/** Show the icon next to the field name. */
		SLATE_ARGUMENT_DEFAULT(bool, bShowFieldIcon) = true;
		/** Sort the field after building the list of fields. */
		SLATE_ARGUMENT_DEFAULT(bool, bSortChildNode) = false;
		/** Show a search box. */
		SLATE_ARGUMENT_DEFAULT(bool, bShowSearchBox) = false;
		/** Focus the search box when the widget is first displayed. */
		SLATE_ARGUMENT_DEFAULT(bool, bFocusSearchBox) = false;
		/** Is selection enabled. */ 
		SLATE_ARGUMENT_DEFAULT(ESelectionMode::Type, SelectionMode) = ESelectionMode::Single;
		/** Which properties/functions to show. FFieldIterator_BlueprintVisible is the default. */
		SLATE_ARGUMENT_DEFAULT(IFieldIterator*, FieldIterator) = nullptr;
		/** Which properties/functions that allow expansion. FFieldIterator_NoExpand is the default. */
		SLATE_ARGUMENT_DEFAULT(IFieldExpander*, FieldExpander) = nullptr;
		/** Callback when a property is modified. */
		SLATE_ARGUMENT_DEFAULT(INotifyHook*, NotifyHook) = nullptr;
		/** Override the color settings of the icons. */
		SLATE_ARGUMENT(TOptional<const FFieldColorSettings>, OverrideIconColorSettings);
		/** Slot for additional widget to go before the search box. */
		SLATE_NAMED_SLOT(FArguments, SearchBoxPreSlot)
		/** Slot for additional widget to go after the search box. */
		SLATE_NAMED_SLOT(FArguments, SearchBoxPostSlot)
		/** Slot for additional widget to go before the field or container widget. */
		SLATE_EVENT(FGetFieldWidget, OnGetPreSlot)
		/** Slot for additional widget to go after the field or container widget. */
		SLATE_EVENT(FGetFieldWidget, OnGetPostSlot)
		/** Context menu widget for the selected item. */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
		/** Delegate to invoke when selection changes. */
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		/** Delegate to invoke when an item is double-clicked. */
		SLATE_EVENT(FOnDoubleClicked, OnDoubleClicked);
		/** Delegate to invoke when we generate the entry for an added container. */
		SLATE_EVENT(FOnGenerateContainer, OnGenerateContainer)
		/** Delegate to invoke when an item is dragged. */
		SLATE_EVENT(FOnDragDetected, OnDragDetected);
	SLATE_END_ARGS()

	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const UScriptStruct* Struct);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const UScriptStruct* Struct, void* Data);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const UClass* Class);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, UObject* ObjectInstance);
	ADVANCEDWIDGETS_API void Construct(const FArguments& InArgs, const UFunction* Function);

	ADVANCEDWIDGETS_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

public:
	ADVANCEDWIDGETS_API FHandle AddContainer(const UScriptStruct* Struct, TOptional<FText> DisplayName = TOptional<FText>());
	ADVANCEDWIDGETS_API FHandle AddContainer(const UClass* Class, TOptional<FText> DisplayName = TOptional<FText>());
	ADVANCEDWIDGETS_API FHandle AddContainer(const UFunction* Function, TOptional<FText> DisplayName = TOptional<FText>());
	ADVANCEDWIDGETS_API FHandle AddInstance(const UScriptStruct* Struct, void* Data, TOptional<FText> DisplayName = TOptional<FText>());
	ADVANCEDWIDGETS_API FHandle AddInstance(UObject* ObjectInstance, TOptional<FText> DisplayName = TOptional<FText>());

	ADVANCEDWIDGETS_API void Remove(FHandle Identifier);
	ADVANCEDWIDGETS_API void RemoveAll();

	ADVANCEDWIDGETS_API TArray<FSelectedItem> GetSelectedItems() const;

	ADVANCEDWIDGETS_API void SetRawFilterText(const FText& InFilterText);
	ADVANCEDWIDGETS_API void SetSelection(FHandle Container, TArrayView<const FFieldVariant> FieldPath);

private:
	void ConstructInternal(const FArguments& InArgs);
	static FHandle MakeContainerIdentifier();
	TSharedPtr<Private::FPropertyViewerImpl> Implementation;
};

} //namespace
