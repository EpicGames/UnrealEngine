// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateRoundedBoxBrush.h"
#include "CoreMinimal.h"
#include "Curves/CurveFloat.h"
#include "Framework/SlateDelegates.h"
#include "Input/Reply.h"
#include "Misc/Attribute.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Styling/StyleColors.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

class FPaintArgs;
class FSlateWindowElementList;
class UCurveFloat;

class SRadialSlider
	: public SLeafWidget
{
public:

	SLATE_BEGIN_ARGS(SRadialSlider)
		: _MouseUsesStep(false)
		, _RequiresControllerLock(true)
		, _Locked(false)
		, _SliderBarColor(FLinearColor::Gray)
		, _SliderProgressColor(FLinearColor::White)
		, _SliderHandleColor(FLinearColor::White)
		, _CenterBackgroundColor(FLinearColor::Transparent)
		, _CenterBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Transparent, FVector2D(90.0f, 90.0f))) // todo: add to a custom radial slider style 
		, _Style(&FCoreStyle::Get().GetWidgetStyle<FSliderStyle>("Slider"))
		, _StepSize(0.01f)
		, _Value(1.f)
		, _bUseCustomDefaultValue(false)
		, _CustomDefaultValue(0.0f)
		, _SliderHandleStartAngle(60.0f)
		, _SliderHandleEndAngle(300.0f)
		, _AngularOffset(0.0f)
		, _HandStartEndRatio(FVector2D(0.0f, 1.0f))
		, _IsFocusable(true)
		, _UseVerticalDrag(false)
		, _ShowSliderHandle(true)
		, _ShowSliderHand(false)
		, _OnMouseCaptureBegin()
		, _OnMouseCaptureEnd()
		, _OnValueChanged()
		{
		}

		/** Sets new value if mouse position is greater/less than half the step size. */
	SLATE_ARGUMENT(bool, MouseUsesStep)

		/** Sets whether we have to lock input to change the slider value. */
		SLATE_ARGUMENT(bool, RequiresControllerLock)

		/** Whether the handle is interactive or fixed. */
		SLATE_ATTRIBUTE(bool, Locked)

		/** The color to draw the slider bar in. */
		SLATE_ATTRIBUTE(FSlateColor, SliderBarColor)

		/** The color to draw completed progress of the slider bar in. */
		SLATE_ATTRIBUTE(FSlateColor, SliderProgressColor)

		/** The color to draw the slider handle in. */
		SLATE_ATTRIBUTE(FSlateColor, SliderHandleColor)

		/** The color to draw the center background in. */
		SLATE_ATTRIBUTE(FSlateColor, CenterBackgroundColor)

		/** The thickness used for the slider bar. For backwards compatibility, this will only be used instead of the bar thickness from Style if it has been manually set with SetThickness. */
		SLATE_ATTRIBUTE(float, Thickness)

		/** Center background image. */
		SLATE_ARGUMENT(FSlateBrush, CenterBackgroundBrush)

		/** The style used to draw the slider. */
		SLATE_STYLE_ARGUMENT(FSliderStyle, Style )

		/** The input mode while using the controller. */
		SLATE_ATTRIBUTE(float, StepSize)

		/** A value that drives where the slider handle appears. Value is normalized between 0 and 1. */
		SLATE_ATTRIBUTE( float, Value )

		/** Whether the slider should draw it's progress bar from a custom value on the slider */
		SLATE_ATTRIBUTE(bool, bUseCustomDefaultValue)

		/** The value where the slider should draw it's progress bar from, independent of direction */
		SLATE_ATTRIBUTE(float, CustomDefaultValue)

		/** A curve that defines how the slider should be sampled. Default is linear.*/
		SLATE_ARGUMENT(FRuntimeFloatCurve, SliderRange)

		/** The angle at which the Slider Handle will start. */
		SLATE_ARGUMENT(float, SliderHandleStartAngle)

		/** The angle at which the Slider Handle will end. */
		SLATE_ARGUMENT(float, SliderHandleEndAngle)

		/** Rotates radial slider by arbitrary offset to support full gamut of configurations */
		SLATE_ARGUMENT(float, AngularOffset)

		/** Start and end of the hand as a ratio to the slider radius (so 0.0 to 1.0 is from the slider center to the handle). */
		SLATE_ARGUMENT(FVector2D, HandStartEndRatio)

		/** Distributes value tags along the slider */
		SLATE_ARGUMENT(TArray<float>, ValueTags)

		/** Sometimes a slider should only be mouse-clickable and never keyboard focusable. */
		SLATE_ARGUMENT(bool, IsFocusable)

		/** Whether the value is changed when dragging vertically as opposed to along the radial curve.  */
		SLATE_ARGUMENT(bool, UseVerticalDrag)

		/** Whether to show the slider thumb. */
		SLATE_ARGUMENT(bool, ShowSliderHandle)

		/** Whether to show the slider hand. */
		SLATE_ARGUMENT(bool, ShowSliderHand)

		/** Invoked when the mouse is pressed and a capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

		/** Invoked when the mouse is released and a capture ends. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

		/** Invoked when the Controller is pressed and capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnControllerCaptureBegin)

		/** Invoked when the controller capture is released.  */
		SLATE_EVENT(FSimpleDelegate, OnControllerCaptureEnd)

		/** Called when the value is changed by the slider. */
		SLATE_EVENT( FOnFloatValueChanged, OnValueChanged )

	SLATE_END_ARGS()

	ADVANCEDWIDGETS_API SRadialSlider();
	ADVANCEDWIDGETS_API virtual ~SRadialSlider();

	/**
	 * Construct the widget.
	 * 
	 * @param InDeclaration A declaration from which to construct the widget.
	 */
	ADVANCEDWIDGETS_API void Construct( const SRadialSlider::FArguments& InDeclaration );

	/** Get the SliderRange attribute */
	FRuntimeFloatCurve GetSliderRange() const { return SliderRange; }

	/** Get the minumum value in Slider Range */
	ADVANCEDWIDGETS_API float GetMinValue() const;

	/** Get the maximum value in Slider Range */
	ADVANCEDWIDGETS_API float GetMaxValue() const;

	/** Get the MinSliderHandleAngle attribute */
	float GetSliderHandleStartAngle() const { return SliderHandleStartAngle; }

	/** Get the MaxSliderHandleAngle attribute */
	float GetSliderHandleEndAngle() const { return SliderHandleEndAngle; }

	/** Get the AngularOffset attribute */
	float GetAngularOffset() const { return AngularOffset; }

	/** Get the ValueTags attribute */
	TArray<float> GetValueTags() const { return ValueTags;	}

	/** Get the Value attribute */
	ADVANCEDWIDGETS_API float GetValue() const;

	/** Get the bUseCustomDefaultValue attribute */
	ADVANCEDWIDGETS_API bool GetUseCustomDefaultValue() const;

	/** Get ths CustomDefaultValue attribute */
	ADVANCEDWIDGETS_API float GetCustomDefaultValue() const;

	/** Get the Value attribute scaled from 0 to 1 */
	ADVANCEDWIDGETS_API float GetNormalizedValue(float RawValue) const;

	/** Get the Slider's Handle position scaled from 0 to 1 */
	ADVANCEDWIDGETS_API float GetNormalizedSliderHandlePosition() const;

	/** Set the Value attribute */
	ADVANCEDWIDGETS_API void SetValue(const TAttribute<float>& InValueAttribute);

	/** Set the bUseCustomDefaultValue attribute */
	ADVANCEDWIDGETS_API void SetUseCustomDefaultValue(const TAttribute<bool>& InValueAttribute);

	/** Set the CustomDefaultValue attribute */
	ADVANCEDWIDGETS_API void SetCustomDefaultValue(const TAttribute<float>& InValueAttribute);
	
	/** Set the SliderRange attribute */
	void SetSliderRange(const FRuntimeFloatCurve& InSliderRange) { SliderRange = InSliderRange; }

	/** Set the SliderHandleStartAngle and SliderHandleEndAngle attributes. If the new SliderHandleStartAngle is more than the new SliderHandleEndAngle, SliderHandleEndAngle will be changed to equal SliderHandleStartAngle. */
	ADVANCEDWIDGETS_API void SetSliderHandleStartAngleAndSliderHandleEndAngle(float InSliderHandleStartAngle, float InSliderHandleEndAngle);
	
	/** Set the AngularOffset attribute */
	void SetAngularOffset(float InAngularOffset) { AngularOffset = InAngularOffset; }

	/** Set the HandStartEndRatio. Clamped to 0.0 to 1.0, and if the start ratio is more than the end ratio, end ratio will be set to the start ratio.  */
	ADVANCEDWIDGETS_API void SetHandStartEndRatio(FVector2D InHandStartEndRatio);

	/** Set the ValueTags attribute */
	void SetValueTags(const TArray<float>& InValueTags) { ValueTags = InValueTags; }

	/** Set the Locked attribute */
	ADVANCEDWIDGETS_API void SetLocked(const TAttribute<bool>& InLocked);
	
	/** Set the SliderBarColor attribute */
	ADVANCEDWIDGETS_API void SetSliderBarColor(FSlateColor InSliderBarColor);
	
	/** Set the SliderProgressColor attribute */
	ADVANCEDWIDGETS_API void SetSliderProgressColor(FSlateColor InSliderProgressColor);

	/** Set the SliderHandleColor attribute */
	ADVANCEDWIDGETS_API void SetSliderHandleColor(FSlateColor InSliderHandleColor);

	/** Set the SliderHandleColor attribute */
	ADVANCEDWIDGETS_API void SetCenterBackgroundColor(FSlateColor InCenterHandleColor);

	/** Set the Thickness attribute. For backward compatibility, Thickness will be used for drawing instead of Style->BarThickness only if it has been set with this method,  */
	ADVANCEDWIDGETS_API void SetThickness(const float InThickness);

	/** Get the StepSize attribute */
	ADVANCEDWIDGETS_API float GetStepSize() const;

	/** Set the StepSize attribute */
	ADVANCEDWIDGETS_API void SetStepSize(const TAttribute<float>& InStepSize);

	/** Set the MouseUsesStep attribute */
	ADVANCEDWIDGETS_API void SetMouseUsesStep(bool MouseUsesStep);

	/** Set the RequiresControllerLock attribute */
	ADVANCEDWIDGETS_API void SetRequiresControllerLock(bool RequiresControllerLock);

	/** Set the UseVerticalDrag attribute */
	ADVANCEDWIDGETS_API void SetUseVerticalDrag(bool UseVerticalDrag);

	/** Set the ShowSliderHandle attribute */
	ADVANCEDWIDGETS_API void SetShowSliderHandle(bool ShowSliderHandle);

	/** Set the ShowSliderHand attribute */
	ADVANCEDWIDGETS_API void SetShowSliderHand(bool ShowSliderHand);

public:

	// SWidget overrides

	ADVANCEDWIDGETS_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	ADVANCEDWIDGETS_API virtual FVector2D ComputeDesiredSize(float) const override;
	ADVANCEDWIDGETS_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	ADVANCEDWIDGETS_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	ADVANCEDWIDGETS_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	ADVANCEDWIDGETS_API virtual FReply OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	ADVANCEDWIDGETS_API virtual FReply OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	ADVANCEDWIDGETS_API virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	ADVANCEDWIDGETS_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	ADVANCEDWIDGETS_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	ADVANCEDWIDGETS_API virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	ADVANCEDWIDGETS_API virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent) override;

	ADVANCEDWIDGETS_API virtual bool SupportsKeyboardFocus() const override;
	ADVANCEDWIDGETS_API virtual bool IsInteractable() const override;
#if WITH_ACCESSIBILITY
	ADVANCEDWIDGETS_API virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override;
#endif

	/** @return Is the handle locked or not? Defaults to false */
	ADVANCEDWIDGETS_API bool IsLocked() const;

protected:

	/**
	 * Commits the specified slider value.
	 *
	 * @param NewValue The value to commit.
	 */
	ADVANCEDWIDGETS_API virtual void CommitValue(float NewValue);

	/**
	 * Calculates the new value based on the given absolute coordinates.
	 *
	 * @param MyGeometry The slider's geometry.
	 * @param AbsolutePosition The absolute position of the slider.
	 * @return The new value.
	 */
	ADVANCEDWIDGETS_API float PositionToValue(const FGeometry& MyGeometry, const FVector2D& AbsolutePosition);
	
	ADVANCEDWIDGETS_API const FSlateBrush* GetBarImage() const;
	ADVANCEDWIDGETS_API const FSlateBrush* GetThumbImage() const;

protected:

	// Holds the style passed to the widget upon construction.
	const FSliderStyle* Style;

	// Holds a flag indicating whether the slider is locked.
	TAttribute<bool> LockedAttribute;

	// Holds the color of the slider bar.
	TAttribute<FSlateColor> SliderBarColor;

	// Holds the color for the completed progress of the slider bar.
	TAttribute<FSlateColor> SliderProgressColor;

	// Holds the color of the slider handle.
	TAttribute<FSlateColor> SliderHandleColor;

	// Holds the color of the center background.
	TAttribute<FSlateColor> CenterBackgroundColor;

	// Center background image brush
	FSlateBrush CenterBackgroundBrush;

	// Thickness used for slider bar instead of Style->BarThickness (see SetThickness)
	TAttribute<TOptional<float>> Thickness;

	// Holds the slider's current value.
	TAttribute<float> ValueAttribute;

	// Whether the slider should draw it's progress bar from a custom value on the slider
	TAttribute<bool> bUseCustomDefaultValue;

	// The value where the slider should draw it's progress bar from, independent of direction
	TAttribute<float> CustomDefaultValue;

	// Holds the initial cursor in case a custom cursor has been specified, so we can restore it after dragging the slider
	EMouseCursor::Type CachedCursor;

	/** The location in screenspace the slider was pressed by a touch */
	FVector2D PressedScreenSpaceTouchDownPosition = FVector2D(0, 0);

	/** Holds the amount to adjust the value by when using a controller or keyboard */
	TAttribute<float> StepSize;

	/**  The angle at which the radial slider should begin */	
	float SliderHandleStartAngle;

	/**  The angle at which the radial slider should end */	
	float SliderHandleEndAngle;

	/**  The angle at which the radial slider should be offset by */	
	float AngularOffset;

	/** Start and end of the hand as a ratio to the slider radius (so 0.0 to 1.0 is from the slider center to the handle). */
	FVector2D HandStartEndRatio;

	/**  The values that should be drawn around the radial slider*/	
	TArray<float> ValueTags;

	/** A curve that defines how the slider should be sampled. Default is linear. */	
	FRuntimeFloatCurve SliderRange;

	// Holds a flag indicating whether a controller/keyboard is manipulating the slider's value. 
	// When true, navigation away from the widget is prevented until a new value has been accepted or canceled. 
	bool bControllerInputCaptured;

	/** Sets new value if mouse position is greater/less than half the step size. */
	bool bMouseUsesStep;

	/** Sets whether we have to lock input to change the slider value. */
	bool bRequiresControllerLock;

	/** When true, this slider will be keyboard focusable. Defaults to false. */
	bool bIsFocusable;

	/** When true, value is changed when dragging vertically as opposed to along the radial curve.  */
	bool bUseVerticalDrag;

	/** Whether to show the slider handle (thumb). */
	bool bShowSliderHandle;

	/** Whether to show the slider hand. */
	bool bShowSliderHand;

private:

	// Resets controller input state. Fires delegates.
	ADVANCEDWIDGETS_API void ResetControllerState();

	// Helper function to convert Absolute Position to Angle
	ADVANCEDWIDGETS_API float GetAngleFromPosition(const FGeometry& MyGeometry, const FVector2D& AbsolutePosition);
	
	// Called on Mouse / Touch input to cache relevant properties
	ADVANCEDWIDGETS_API void OnInputStarted(const FGeometry& MyGeometry, const FVector2D& InputAbsolutePosition);

	// Helper function for adding slider points to a slider points array
	ADVANCEDWIDGETS_API void AddSliderPointToArray(TArray<FVector2D>& SliderPoints, const bool bIsUnique, const FVector2D& SliderPoint) const;

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when the mouse is let up and a capture ends.
	FSimpleDelegate OnMouseCaptureEnd;

	// Holds a delegate that is executed when capture begins for controller or keyboard.
	FSimpleDelegate OnControllerCaptureBegin;

	// Holds a delegate that is executed when capture ends for controller or keyboard.
	FSimpleDelegate OnControllerCaptureEnd;

	// Holds a delegate that is executed when the slider's value changed.
	FOnFloatValueChanged OnValueChanged;

	// Holds the current interaction's unclamped input angle.
	float AbsoluteInputAngle;

	// Stores the previous absolute position to support calculating rotational delta for relative input
	FVector2D PreviousAbsolutePosition;

	// Settings for behavior when IsUsingVerticalDrag is true
	// For when UseVerticalDrag is true, whether we're fine tuning the value 
	bool bIsUsingFineTune;

	// The key to use when fine tuning vertical drag 
	FKey FineTuneKey;

	float VerticalDragMouseSpeedNormal = 0.2f;
	float VerticalDragMouseSpeedFineTune = 0.05f;
	float VerticalDragPixelDelta = 50.0f;
};
